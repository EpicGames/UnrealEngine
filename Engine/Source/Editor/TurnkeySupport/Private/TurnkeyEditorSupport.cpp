// Copyright Epic Games, Inc. All Rights Reserved.

#include "TurnkeyEditorSupport.h"

#if UE_WITH_TURNKEY_SUPPORT

#include "Internationalization/Text.h"
#include "ITurnkeyIOModule.h"
#include "Misc/AssertionMacros.h"

#if WITH_EDITOR
#include "UnrealEdMisc.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"
#include "PlatformInfo.h"
#include "InstalledPlatformInfo.h"
#include "Misc/MessageDialog.h"
#include "IUATHelperModule.h"
#include "Interfaces/IProjectTargetPlatformEditorModule.h"
#include "Dialogs/Dialogs.h"
#include "Async/Async.h"
#include "GameProjectGenerationModule.h"
#include "ISettingsModule.h"
#include "ISettingsEditorModule.h"
#include "Settings/EditorExperimentalSettings.h"
#include "Settings/LevelEditorPlaySettings.h"
#include "Interfaces/IMainFrameModule.h"
#include "ToolMenus.h"
#include "FileHelpers.h"
#include "Settings/ProjectPackagingSettings.h"
#include "Settings/PlatformsMenuSettings.h"
#include "Containers/Set.h"

namespace
{
TSet<FString> VerifiedPlatformsAndDevices;
}
#endif

#define LOCTEXT_NAMESPACE "FTurnkeyEditorSupport"

FString FTurnkeyEditorSupport::GetUATOptions()
{
#if WITH_EDITOR
	FString Options;
	Options += FString::Printf(TEXT(" -unrealexe=\"%s\""), *FUnrealEdMisc::Get().GetExecutableForCommandlets());

	return Options;
#else
	return TEXT("");
#endif
}


void FTurnkeyEditorSupport::AddEditorOptions(FToolMenuSection& Section)
{
#if WITH_EDITOR
	Section.AddSeparator("TurnkeyOptionsSeparator");

	Section.AddMenuEntry(
		NAME_None,
		LOCTEXT("OpenPackagingSettings", "Packaging Settings..."),
		LOCTEXT("OpenPackagingSettings_ToolTip", "Opens the settings for project packaging."),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateLambda([] { FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Project", "Project", "Packaging"); }))
	);

	// use AddDynamicEntry to be able to get a MenuBuilder for external code that needs one
	Section.AddDynamicEntry(NAME_None, FNewToolMenuDelegateLegacy::CreateLambda([](FMenuBuilder& MenuBuilder, UToolMenu* Menu)
		{
			FModuleManager::LoadModuleChecked<IProjectTargetPlatformEditorModule>("ProjectTargetPlatformEditor").AddOpenProjectTargetPlatformEditorMenuItem(MenuBuilder);
		})
	);
#endif
}

void FTurnkeyEditorSupport::PrepareToLaunchRunningMap(const FString& DeviceId, const FString& DeviceName)
{
#if WITH_EDITOR
	ULevelEditorPlaySettings* PlaySettings = GetMutableDefault<ULevelEditorPlaySettings>();

	PlaySettings->LastExecutedLaunchModeType = LaunchMode_OnDevice;
	PlaySettings->LastExecutedLaunchDevice = DeviceId;
	PlaySettings->LastExecutedLaunchName = DeviceName;

	PlaySettings->PostEditChange();

	PlaySettings->SaveConfig();
#endif
}

void FTurnkeyEditorSupport::LaunchRunningMap(const FString& DeviceId, const FString& DeviceName, const FString& ProjectPath, bool bUseTurnkey, bool bOnSimulator, FStringView Architecture)
{
#if WITH_EDITOR
	FTargetDeviceId TargetDeviceId;
	if (FTargetDeviceId::Parse(DeviceId, TargetDeviceId))
	{
		const PlatformInfo::FTargetPlatformInfo* PlatformInfo = nullptr;
		if (FApp::IsInstalled())
		{
			PlatformInfo = PlatformInfo::FindPlatformInfo(*TargetDeviceId.GetPlatformName());
		}
		else
		{
			PlatformInfo = PlatformInfo::FindPlatformInfo(GetDefault<UPlatformsMenuSettings>()->GetTargetFlavorForPlatform(*TargetDeviceId.GetPlatformName()));
		}
					
		FString UBTPlatformName = PlatformInfo->DataDrivenPlatformInfo->UBTPlatformString;
		FString IniPlatformName = PlatformInfo->IniPlatformName.ToString();

		check(PlatformInfo);

		if (FInstalledPlatformInfo::Get().IsPlatformMissingRequiredFile(UBTPlatformName))
		{
			if (!FInstalledPlatformInfo::OpenInstallerOptions())
			{
				FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("MissingPlatformFilesLaunch", "Missing required files to launch on this platform."));
			}
			return;
		}

		if (FModuleManager::LoadModuleChecked<IProjectTargetPlatformEditorModule>("ProjectTargetPlatformEditor").ShowUnsupportedTargetWarning(*TargetDeviceId.GetPlatformName()))
		{
			GUnrealEd->CancelPlayingViaLauncher();

			FRequestPlaySessionParams::FLauncherDeviceInfo DeviceInfo;
			DeviceInfo.DeviceId = DeviceId;
			DeviceInfo.DeviceName = DeviceName;
			// @todo turnkey: we set this to false because we will kick off a Turnkey run before cooking, etc, to get an early warning. however, if it's too difficult
			// to get an error back from CreateUatTask, then we should set this to bUseTurnkey and remove the block below, and let the code in FLauncherWorker::CreateAndExecuteTasks handle it
			DeviceInfo.bUpdateDeviceFlash = false;
			DeviceInfo.bIsSimulator = bOnSimulator;
			DeviceInfo.Architecture = Architecture;

			FRequestPlaySessionParams SessionParams;
			SessionParams.SessionDestination = EPlaySessionDestinationType::Launcher;
			SessionParams.LauncherTargetDevice = MoveTemp(DeviceInfo);

			// if we want to check device flash before we start cooking, kick it off now. we could delay this 
			if (bUseTurnkey)
			{
				const FString& RealDeviceName = TargetDeviceId.GetDeviceName();
				const bool bSkipPlatformCheck = VerifiedPlatformsAndDevices.Contains(UBTPlatformName);
				const bool bSkipDeviceCheck = VerifiedPlatformsAndDevices.Contains(RealDeviceName);

				FString CommandLine;
				if (bSkipPlatformCheck && bSkipDeviceCheck)
				{
					GUnrealEd->RequestPlaySession(SessionParams);
					return;
				}
				else if (bSkipPlatformCheck && !bSkipDeviceCheck)
				{
					CommandLine = FString::Printf(TEXT("Turnkey -command=VerifySdk -UpdateIfNeeded -platform=%s -SkipPlatform -noturnkeyvariables -device=%s -utf8output -WaitForUATMutex %s %s"), *UBTPlatformName, *RealDeviceName, *PlatformInfo->UATCommandLine, *ITurnkeyIOModule::Get().GetUATParams());
				}
				else
				{
					CommandLine = FString::Printf(TEXT("Turnkey -command=VerifySdk -UpdateIfNeeded -platform=%s -noturnkeyvariables -device=%s -utf8output -WaitForUATMutex %s %s"), *UBTPlatformName, *RealDeviceName, *PlatformInfo->UATCommandLine, *ITurnkeyIOModule::Get().GetUATParams());
				}
				
				if (!ProjectPath.IsEmpty())
				{
					CommandLine = FString::Printf(TEXT(" -ScriptsForProject=\"%s\" %s -project=\"%s\""), *ProjectPath, *CommandLine, *ProjectPath);
				}
				FText TaskName = LOCTEXT("VerifyingSDK", "Verifying SDK and Device");

				IUATHelperModule::Get().CreateUatTask(CommandLine, FText::FromString(IniPlatformName), TaskName, TaskName, FAppStyle::Get().GetBrush(TEXT("MainFrame.PackageProject")), nullptr,
					[SessionParams, RealDeviceName, UBTPlatformName](FString Result, double)
					{
						// unfortunate string comparison for success
						bool bWasSuccessful = Result == TEXT("Completed");
						AsyncTask(ENamedThreads::GameThread, [SessionParams, bWasSuccessful, RealDeviceName, UBTPlatformName]()
							{
								if (bWasSuccessful)
								{
									GUnrealEd->RequestPlaySession(SessionParams);
									VerifiedPlatformsAndDevices.Add(RealDeviceName);
									VerifiedPlatformsAndDevices.Add(UBTPlatformName);
								}
								else
								{
									TSharedRef<SWindow> Win = OpenMsgDlgInt_NonModal(EAppMsgType::YesNo, LOCTEXT("SDKCheckFailed", "SDK Verification failed. Would you like to attempt the Launch On anyway?"), LOCTEXT("SDKCheckFailedTitle", "SDK Verification"),
										FOnMsgDlgResult::CreateLambda([SessionParams](const TSharedRef<SWindow>&, EAppReturnType::Type Choice)
											{
												if (Choice == EAppReturnType::Yes)
												{
													GUnrealEd->RequestPlaySession(SessionParams);
												}
											}));
									Win->ShowWindow();
								}
							});
					});
			}
			else
			{
				GUnrealEd->RequestPlaySession(SessionParams);
			}
		}
	}
#endif
}

void FTurnkeyEditorSupport::SaveAll()
{
#if WITH_EDITOR
	FEditorFileUtils::SaveDirtyPackages(false /*bPromptUserToSave*/
		, true /*bSaveMapPackages*/
		, true /*bSaveContentPackages*/
		, false /*bFastSave*/
		, false /*bNotifyNoPackagesSaved*/
		, false /*bCanBeDeclined*/);
#else
	unimplemented();
#endif
}

bool FTurnkeyEditorSupport::DoesProjectHaveCode()
{
#if WITH_EDITOR
	FGameProjectGenerationModule& GameProjectModule = FModuleManager::LoadModuleChecked<FGameProjectGenerationModule>(TEXT("GameProjectGeneration"));
	return GameProjectModule.Get().ProjectHasCodeFiles();
#else
	unimplemented();
	return false;
#endif
}

void FTurnkeyEditorSupport::RunUAT(const FString& CommandLine, const FText& PlatformDisplayName, const FText& TaskName, const FText& TaskShortName, const FSlateBrush* TaskIcon, const TArray<FAnalyticsEventAttribute>* OptionalAnalyticsParamArray, TFunction<void(FString, double)> ResultCallback, const FText& ExtraSubText)
{
#if WITH_EDITOR
	const FString ResultLocation;
	IUATHelperModule::Get().CreateUatTask(CommandLine, PlatformDisplayName, TaskName, TaskShortName, TaskIcon, OptionalAnalyticsParamArray, ResultCallback, ResultLocation, ExtraSubText);
#else
	unimplemented();
#endif
}




bool FTurnkeyEditorSupport::ShowOKCancelDialog(FText Message, FText Title)
{
#if WITH_EDITOR
	FSuppressableWarningDialog::FSetupInfo Info(Message, Title, TEXT("TurkeyEditorDialog"));

	Info.ConfirmText = LOCTEXT("TurnkeyDialog_Confirm", "Continue");
	Info.CancelText = LOCTEXT("TurnkeyDialogK_Cancel", "Cancel");
	FSuppressableWarningDialog Dialog(Info);

	return Dialog.ShowModal() != FSuppressableWarningDialog::EResult::Cancel;
#else
	unimplemented();
	return false;
#endif
}

void FTurnkeyEditorSupport::ShowRestartToast()
{
#if WITH_EDITOR
	// show restart dialog
	FModuleManager::GetModuleChecked<ISettingsEditorModule>("SettingsEditor").OnApplicationRestartRequired();
#endif
}

void FTurnkeyEditorSupport::ShowRebuildToast()
{
#if WITH_EDITOR
	// show a dialog asking for the editor to be rebuilt
	// @todo: if the source code is available should display a 'rebuild now' button
	FNotificationInfo* Info = new FNotificationInfo( LOCTEXT("Turnkey_RebuildRequired", "The editor must be rebuilt to include support for the new platform") );
	Info->ExpireDuration = 8.0f;
	Info->Image = FAppStyle::GetBrush("Icons.InfoWithColor");
	FSlateNotificationManager::Get().QueueNotification(Info);
#endif
}

bool FTurnkeyEditorSupport::CheckSupportedPlatforms(FName IniPlatformName)
{
#if WITH_EDITOR
	return FModuleManager::LoadModuleChecked<IProjectTargetPlatformEditorModule>("ProjectTargetPlatformEditor").ShowUnsupportedTargetWarning(IniPlatformName);
#else
	return true;
#endif
}

void FTurnkeyEditorSupport::ShowInstallationHelp(FName IniPlatformName, FString DocLink)
{
#if WITH_EDITOR
	// broadcast this, and assume someone will pick it up
	IMainFrameModule& MainFrameModule = FModuleManager::GetModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
	MainFrameModule.BroadcastMainFrameSDKNotInstalled(IniPlatformName.ToString(), DocLink);
#else
	unimplemented();
#endif
}

bool FTurnkeyEditorSupport::IsPIERunning()
{
#if WITH_EDITOR
	return GEditor->PlayWorld != NULL;
#else
	return false;
#endif
}


#undef LOCTEXT_NAMESPACE

#endif // UE_WITH_TURNKEY_SUPPORT
