// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/ITurnkeySupportModule.h"

#if UE_WITH_TURNKEY_SUPPORT

class FTurnkeyEditorSupport
{
public:

	static FString GetUATOptions();

	static void PrepareToLaunchRunningMap(const FString& DeviceId, const FString& DeviceName);
	static void LaunchRunningMap(const FString& DeviceId, const FString& DeviceName, const FString& ProjectPath, bool bUseTurnkey, bool bOnSimulator = false, FStringView Architecture = {});
	static void AddEditorOptions(struct FToolMenuSection& MenuBuilder);
	
	static void SaveAll();
	static bool DoesProjectHaveCode();
	static void RunUAT(const FString& CommandLine, const FText& PlatformDisplayName, const FText& TaskName, const FText& TaskShortName, const struct FSlateBrush* TaskIcon, const TArray<struct FAnalyticsEventAttribute>* OptionalAnalyticsParamArray = nullptr, TFunction<void(FString, double)> ResultCallback=TFunction<void(FString, double)>(), const FText& ExtraSubText = FText::GetEmpty());

	static bool ShowOKCancelDialog(FText Message, FText Title);
	static void ShowRestartToast();
	static void ShowRebuildToast();
	static bool CheckSupportedPlatforms(FName IniPlatformName);
	static void ShowInstallationHelp(FName IniPlatformName, FString DocLink);
	static bool IsPIERunning();
};

#endif // UE_WITH_TURNKEY_SUPPORT