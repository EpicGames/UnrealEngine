// Copyright Epic Games, Inc. All Rights Reserved.

#include "TurnkeySupportModule.h"

#if !UE_WITH_TURNKEY_SUPPORT
class FTurnkeySupportModuleEmpty : public IModuleInterface {};
IMPLEMENT_MODULE(FTurnkeySupportModuleEmpty, TurnkeySupport)
#else // UE_WITH_TURNKEY_SUPPORT

#include "TurnkeySupport.h"

#include "SlateOptMacros.h"
#include "UObject/Package.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SToolTip.h"
#include "MessageLogModule.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Misc/MessageDialog.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "ITargetDeviceServicesModule.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "Async/Async.h"
#include "HAL/Event.h"
#include "Misc/FileHelper.h"
#include "Misc/ConfigCacheIni.h"
#include "Delegates/Delegate.h"
#include "Interfaces/TargetDeviceId.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Interfaces/IProjectManager.h"
#include "Interfaces/IPluginManager.h"
#include "SourceControlHelpers.h"
#include "ISourceControlModule.h"
#include "ITargetDeviceProxy.h"
#include "ITargetDeviceServicesModule.h"
#include "Settings/ProjectPackagingSettings.h"
#include "Settings/PlatformsMenuSettings.h"
#include "IDesktopPlatform.h"
#include "DesktopPlatformModule.h"
#include "PlatformInfo.h"
#include "InstalledPlatformInfo.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "DerivedDataCacheInterface.h"
#include "Misc/MonitoredProcess.h"
#include "CookerSettings.h"
#include "UObject/UObjectIterator.h"
#include "ToolMenus.h"
#include "TurnkeyEditorSupport.h"
#include "ITurnkeyIOModule.h"
#include "AnalyticsEventAttribute.h"
#if WITH_EDITOR
#include "LevelEditor.h"
#include "Experimental/ZenServerInterface.h"
#endif

#include "Misc/App.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/AppStyle.h"

#if WITH_ENGINE
#include "RenderUtils.h"
#endif

DEFINE_LOG_CATEGORY(LogTurnkeySupport);
#define LOCTEXT_NAMESPACE "FTurnkeySupportModule"


#define ALLOW_CONTROL_TO_COPY_COMMANDLINE 0

namespace 
{
	FCriticalSection GTurnkeySection;

	// The host platform's IniPlatformName, used to pick defaults.
#if PLATFORM_WINDOWS
	const FName GHostIniPlatformName("Windows");
#elif PLATFORM_MAC
	const FName GHostIniPlatformName("Mac");
#else
	// NAME_None disables this behavior and uses the fallback behavior for choosing a default.
	const FName GHostIniPlatformName(NAME_None);
#endif
}

static FString GetProjectPathForTurnkey()
{
	if (FPaths::IsProjectFilePathSet())
	{
		return FPaths::ConvertRelativePathToFull(FPaths::GetProjectFilePath());
	}
	if (FApp::HasProjectName())
	{
		FString ProjectPath = FPaths::ProjectDir() / FApp::GetProjectName() + TEXT(".uproject");
		if (FPaths::FileExists(ProjectPath))
		{
			return ProjectPath;
		}
		ProjectPath = FPaths::RootDir() / FApp::GetProjectName() / FApp::GetProjectName() + TEXT(".uproject");
		if (FPaths::FileExists(ProjectPath))
		{
			return ProjectPath;
		}
	}
	return FString();
}

enum class EPrepareContentMode : uint8
{
	CookOnly,
	Package,
	PrepareForDebugging,
};

class FTurnkeySupportCallbacks
{
protected:
	static const TCHAR* GetUATCompilationFlags()
	{
		// We never want to compile editor targets when invoking UAT in this context.
		// If we are installed or don't have a compiler, we must assume we have a precompiled UAT.
		return TEXT("-nocompileeditor -skipbuildeditor");
	}


	static bool ShowBadSDKDialog(FName IniPlatformName)
	{
		// Don't show the warning during automation testing; the dlg is modal and blocks
		if (!GIsAutomationTesting)
		{
			FFormatNamedArguments Args;
			//		Args.Add(TEXT("DisplayName"), PlatformInfo->DisplayName);
			Args.Add(TEXT("DisplayName"), FText::FromName(IniPlatformName));
			FText WarningText = FText::Format(LOCTEXT("BadSDK_Message", "The SDK for {DisplayName} is not installed properly, which is needed to generate data. Check the SDK section of the Launch On menu in the main toolbar to update SDK.\n\nWould you like to attempt to continue anyway?"), Args);

			bool bClickedOK = FTurnkeyEditorSupport::ShowOKCancelDialog(WarningText, LOCTEXT("BadSDK_Title", "SDK Not Setup"));
			return bClickedOK;

		}

		return true;
	}


	static bool ShouldBuildProject(const UProjectPackagingSettings* PackagingSettings, const ITargetPlatform* TargetPlatform)
	{
		const UProjectPackagingSettings::FConfigurationInfo& ConfigurationInfo = UProjectPackagingSettings::ConfigurationInfo[(int)PackagingSettings->BuildConfiguration];

		// Get the target to build
		const FTargetInfo* Target = PackagingSettings->GetBuildTargetInfo();

		// Only build if the user elects to do so
		bool bBuild = false;
		if (PackagingSettings->Build == EProjectPackagingBuild::Always)
		{
			bBuild = true;
		}
		else if (PackagingSettings->Build == EProjectPackagingBuild::Never)
		{
			bBuild = false;
		}
		else if (PackagingSettings->Build == EProjectPackagingBuild::IfProjectHasCode)
		{
			bBuild = true;
			if (FApp::GetEngineIsPromotedBuild())
			{
				FString BaseDir;

				// Get the target name
				FString TargetName;
				if (Target == nullptr)
				{
					TargetName = TEXT("UnrealGame");
				}
				else
				{
					TargetName = Target->Name;
				}

				// Get the directory containing the receipt for this target, depending on whether the project needs to be built or not
				FString ProjectDir = FPaths::GetPath(FPaths::GetProjectFilePath());
				if (Target != nullptr && FPaths::IsUnderDirectory(Target->Path, ProjectDir))
				{
					UE_LOGF(LogTurnkeySupport, Log, "Selected target: %ls", *Target->Name);
					BaseDir = ProjectDir;
				}
				else
				{
					FText Reason;

					if (TargetPlatform->RequiresTempTarget(FTurnkeyEditorSupport::DoesProjectHaveCode(), ConfigurationInfo.Configuration, false, Reason))
					{
						UE_LOGF(LogTurnkeySupport, Log, "Project requires temp target (%ls)", *Reason.ToString());
						BaseDir = ProjectDir;
					}
					else
					{
						UE_LOGF(LogTurnkeySupport, Log, "Project does not require temp target");
						BaseDir = FPaths::EngineDir();
					}
				}

				// Check if the receipt is for a matching promoted target
				FString UBTPlatformName = TargetPlatform->GetTargetPlatformInfo().DataDrivenPlatformInfo->UBTPlatformString;

				extern LAUNCHERSERVICES_API bool HasPromotedTarget(const TCHAR * BaseDir, const TCHAR * TargetName, const TCHAR * Platform, EBuildConfiguration Configuration, const TCHAR * Architecture);
				if (HasPromotedTarget(*BaseDir, *TargetName, *UBTPlatformName, ConfigurationInfo.Configuration, nullptr))
				{
					bBuild = false;
				}
			}
		}
		else if (PackagingSettings->Build == EProjectPackagingBuild::IfEditorWasBuiltLocally)
		{
			bBuild = !FApp::GetEngineIsPromotedBuild() && !FApp::IsEngineInstalled();
		}

		return bBuild;
	}

public:


	static FString GetLogAndReportCommandline(FString& LogFilename, FString& ReportFilename)
	{
		static int ReportIndex = 0;

		LogFilename = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::ProjectIntermediateDir(), *FString::Printf(TEXT("TurnkeyLog_%d.log"), ReportIndex)));
		ReportFilename = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::ProjectIntermediateDir(), *FString::Printf(TEXT("TurnkeyReport_%d.log"), ReportIndex++)));

		return FString::Printf(TEXT("-ReportFilename=\"%s\" -log=\"%s\""), *ReportFilename, *LogFilename);
	}

	static void OpenProjectLauncher()
	{
		FGlobalTabmanager::Get()->TryInvokeTab(FTabId("ProjectLauncher"));
	}

	static void OpenDeviceManager()
	{
		FGlobalTabmanager::Get()->TryInvokeTab(FTabId("DeviceManager"));
	}

	static bool CanCookOrPackage(FName IniPlatformName, EPrepareContentMode Mode)
	{
		if (GetTargetPlatformManager()->FindTargetPlatform(IniPlatformName.ToString()) == nullptr)
		{
			return false;
		}

		return true;
	}

	static UProjectPackagingSettings* GetPackagingSettingsForPlatform(FName IniPlatformName)
	{
		FString PlatformString = IniPlatformName.ToString();
		UProjectPackagingSettings* PackagingSettings = nullptr;
		for (TObjectIterator<UProjectPackagingSettings> Itr; Itr; ++Itr)
		{
			if (Itr->GetConfigPlatform() == PlatformString)
			{
				PackagingSettings = *Itr;
				break;
			}
		}
		if (PackagingSettings == nullptr)
		{
			PackagingSettings = NewObject<UProjectPackagingSettings>(GetTransientPackage());
			// Prevent object from being GCed.
			PackagingSettings->AddToRoot();
			// make sure any changes to DefaultGame are updated in this class
			PackagingSettings->LoadSettingsForPlatform(PlatformString);
		}

		// make sure any flushed settings are reloaded
		PackagingSettings->ReloadConfig();

		return PackagingSettings;
	}

	static void CookOrPackage(FName IniPlatformName, const FString& BuildTargetName, EPrepareContentMode Mode)
	{
		TArray<FAnalyticsEventAttribute> AnalyticsParamArray;

		// get a in-memory defaults which will have the user-settings, like the per-platform config/target platform stuff
		const UProjectPackagingSettings* PackagingSettings = GetPackagingSettingsForPlatform(IniPlatformName);
		UPlatformsMenuSettings* PlatformsSettings = GetMutableDefault<UPlatformsMenuSettings>();

		// installed builds only support standard Game type builds (not Client, Server, etc) so instead of looking up a setting that the user can't set, 
		// always use the base PlatformInfo for Game builds, which will be named the same as the platform itself
		const PlatformInfo::FTargetPlatformInfo* PlatformInfo = nullptr;
		if(FApp::IsInstalled())
		{
			PlatformInfo = PlatformInfo::FindPlatformInfo(IniPlatformName);
		}
		else
		{
			PlatformInfo = PlatformInfo::FindPlatformInfo(PlatformsSettings->GetTargetFlavorForPlatform(IniPlatformName));
		}
		// this is unexpected to be able to happen, but it could if there was a bad value saved in the UProjectPackagingSettings - if this trips, we should handle errors
		check(PlatformInfo != nullptr);

		const FString UBTPlatformString = PlatformInfo->DataDrivenPlatformInfo->UBTPlatformString;
		const FString ProjectPath = GetProjectPathForTurnkey();

		// check that we can proceed
		{
			if (FInstalledPlatformInfo::Get().IsPlatformMissingRequiredFile(UBTPlatformString))
			{
				if (!FInstalledPlatformInfo::OpenInstallerOptions())
				{
					FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("MissingPlatformFilesCook", "Missing required files to cook for this platform."));
				}
				return;
			}

			if (!FTurnkeyEditorSupport::CheckSupportedPlatforms(IniPlatformName))
			{
				return;
			}

			if (ITurnkeySupportModule::Get().GetSdkInfo(IniPlatformName).Status != ETurnkeyPlatformSdkStatus::Valid && ShowBadSDKDialog(IniPlatformName) == false)
			{
				return;
			}
		}

		// force a save of dirty packages before proceeding to run UAT
		// this may delete UProjectPackagingSettings , don't hold it across this call
		FTurnkeyEditorSupport::SaveAll();

		// basic BuildCookRun params we always want
		FString BuildCookRunParams = FString::Printf(TEXT("-nop4 -utf8output %s -cook "), GetUATCompilationFlags());


		// set locations to engine and project
		if (!ProjectPath.IsEmpty())
		{
			BuildCookRunParams += FString::Printf(TEXT(" -project=\"%s\""), *ProjectPath);
		}

		bool bIsProjectBuildTarget = false;
		const FTargetInfo* BuildTargetInfo = PlatformsSettings->GetBuildTargetInfo(BuildTargetName, bIsProjectBuildTarget);

		// Only add the -Target=... argument for code projects. Content projects will return UnrealGame/UnrealClient/UnrealServer here, but
		// may need a temporary target generated to enable/disable plugins. Specifying -Target in these cases will cause packaging to fail,
		// since it'll have a different name.
		if (BuildTargetInfo && bIsProjectBuildTarget)
		{
			BuildCookRunParams += FString::Printf(TEXT(" -target=%s"), *BuildTargetInfo->Name);
		}

		// let the editor add options (-unrealexe in particular)
		{
			BuildCookRunParams += FString::Printf(TEXT(" %s"), *FTurnkeyEditorSupport::GetUATOptions());
		}

		// set the platform we are preparing content for
		{
			BuildCookRunParams += FString::Printf(TEXT(" -platform=%s"), *UBTPlatformString);
		}

		// Append any extra UAT flags specified for this platform flavor
		if (!PlatformInfo->UATCommandLine.IsEmpty())
		{
			BuildCookRunParams += FString::Printf(TEXT(" %s"), *PlatformInfo->UATCommandLine);
		}

		// optional settings
		if (PackagingSettings->bSkipEditorContent)
		{
			BuildCookRunParams += TEXT(" -SkipCookingEditorContent");
		}
		if (FDerivedDataCacheInterface* DDC = TryGetDerivedDataCache())
		{
			const TCHAR* GraphName = DDC->GetGraphName();
			if (FCString::Strcmp(GraphName, DDC->GetDefaultGraphName()))
			{
				BuildCookRunParams += FString::Printf(TEXT(" -DDC=%s"), DDC->GetGraphName());
			}
		}
		if (FApp::IsEngineInstalled())
		{
			BuildCookRunParams += TEXT(" -installed");
		}

		if (PackagingSettings->GetUseZenStoreEffective())
		{
#if WITH_EDITOR
			static UE::Zen::FScopeZenService TurnkeyStaticZenService;
#endif
			BuildCookRunParams += TEXT(" -zenstore");
		}

		// don't report warnings and errors twice
		BuildCookRunParams += TEXT(" -SkipCookingErrorSummary");

		// use structured logging if available
		BuildCookRunParams += TEXT(" -JsonStdOut");

		// gather analytics
		const ITargetPlatform* TargetPlatform = GetTargetPlatformManager()->FindTargetPlatform(PlatformInfo->Name);
		TargetPlatform->GetPlatformSpecificProjectAnalytics( AnalyticsParamArray );


		// append the architecture
		FString Architecture = PlatformsSettings->GetArchitectureForPlatform(IniPlatformName);
		if (!Architecture.IsEmpty())
		{
			if (BuildTargetInfo)
			{
				switch (BuildTargetInfo->Type)
				{
					case EBuildTargetType::Server:
					{
						BuildCookRunParams += FString::Printf(TEXT(" -serverarchitecture=%s"), *Architecture);
					}
					break;

					case EBuildTargetType::Editor:
					{
						BuildCookRunParams += FString::Printf(TEXT(" -editorarchitecture=%s"), *Architecture);
					}
					break;

					default:
					{
						BuildCookRunParams += FString::Printf(TEXT(" -clientarchitecture=%s"), *Architecture);
					}
					break;
				}
			}
			else
			{
				// Added in 5.7.2 to help diagnose UE-316569, a crash at the switch statement above because BuildTargetInfo is null.
				ensureMsgf(BuildTargetInfo, TEXT("CookOrPackage BuildTargetName='%s' IniPlatformName='%s' Mode='%i' BuildCookRunParams='%s 'where BuildTargetInfo is null and architecture '%s' is specified.  One way to do this is to add c++ code to a project, but not build the project editor, then package. Returning without cook or package instead of asserting."), *BuildTargetName, *IniPlatformName.ToString(), EnumToUnderlyingType(Mode), *BuildCookRunParams, *Architecture);
				return;
			}
		}

		// per mode settings
		FText ContentPrepDescription;
		FText ContentPrepTaskName;
		const FSlateBrush* ContentPrepIcon = nullptr;
		FText RoyaltySubText;
		if (Mode == EPrepareContentMode::Package)
		{
			ContentPrepDescription = LOCTEXT("PackagingProjectTaskName", "Packaging project");
			ContentPrepTaskName = LOCTEXT("PackagingTaskName", "Packaging");
			ContentPrepIcon = FAppStyle::Get().GetBrush(TEXT("MainFrame.PackageProject"));

			// let the user pick a target directory
			if (PlatformsSettings->StagingDirectory.Path.IsEmpty())
			{
				PlatformsSettings->StagingDirectory.Path = FPaths::ProjectDir();
			}

			FString OutFolderName;

			if (!FDesktopPlatformModule::Get()->OpenDirectoryDialog(FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr), LOCTEXT("PackageDirectoryDialogTitle", "Package project...").ToString(), PlatformsSettings->StagingDirectory.Path, OutFolderName))
			{
				return;
			}

			PlatformsSettings->StagingDirectory.Path = OutFolderName;
			PlatformsSettings->SaveConfig();


			BuildCookRunParams += TEXT(" -stage -archive -package");

			if (ShouldBuildProject(PackagingSettings, TargetPlatform))
			{
				BuildCookRunParams += TEXT(" -build");
			}

			if (PackagingSettings->FullRebuild)
			{
				BuildCookRunParams += TEXT(" -clean");
			}

			// Pak file(s) must be used when using container file(s)
			if (PackagingSettings->UsePakFile || PackagingSettings->bUseIoStore)
			{
				BuildCookRunParams += TEXT(" -pak");
				if (PackagingSettings->bUseIoStore)
				{
					BuildCookRunParams += TEXT(" -iostore");
				}

				if (PackagingSettings->bCompressed)
				{
					BuildCookRunParams += TEXT(" -compressed");
				}
			}

			if (PackagingSettings->IncludePrerequisites)
			{
				BuildCookRunParams += TEXT(" -prereqs");
			}

			if (!PackagingSettings->ApplocalPrerequisitesDirectory.Path.IsEmpty())
			{
				BuildCookRunParams += FString::Printf(TEXT(" -applocaldirectory=\"%s\""), *(PackagingSettings->ApplocalPrerequisitesDirectory.Path));
			}
			else if (PackagingSettings->IncludeAppLocalPrerequisites)
			{
				BuildCookRunParams += TEXT(" -applocaldirectory=\"$(EngineDir)/Binaries/ThirdParty/AppLocalDependencies\"");
			}

			BuildCookRunParams += FString::Printf(TEXT(" -archivedirectory=\"%s\""), *PlatformsSettings->StagingDirectory.Path);

			if (PackagingSettings->ForDistribution)
			{
				BuildCookRunParams += TEXT(" -distribution");
			}

			if (PackagingSettings->bGenerateChunks)
			{
				BuildCookRunParams += TEXT(" -manifests");
			}

			// Whether to include the crash reporter.
			if (PackagingSettings->IncludeCrashReporter && PlatformInfo->DataDrivenPlatformInfo->bCanUseCrashReporter)
			{
				BuildCookRunParams += TEXT(" -CrashReporter");
			}

			if (PackagingSettings->bBuildHttpChunkInstallData)
			{
				BuildCookRunParams += FString::Printf(TEXT(" -manifests -createchunkinstall -chunkinstalldirectory=\"%s\" -chunkinstallversion=%s"), *(PackagingSettings->HttpChunkInstallDataDirectory.Path), *(PackagingSettings->HttpChunkInstallDataVersion));
			}

			EProjectPackagingBuildConfigurations BuildConfig = PlatformsSettings->GetPackageBuildConfiguration();
			// if PPBC_MAX is set, then the project default should be used instead of the per platform build config
			if (BuildConfig == EProjectPackagingBuildConfigurations::PPBC_MAX)
			{
				BuildConfig = PackagingSettings->BuildConfiguration;
			}

			// when distribution is set, always package in shipping, which overrides the per platform build config
			if (PackagingSettings->ForDistribution)
			{
				BuildConfig = EProjectPackagingBuildConfigurations::PPBC_Shipping;

				constexpr const TCHAR* ReleaseUrl = TEXT("https://www.unrealengine.com/release");
				RoyaltySubText = FText::Format(
					LOCTEXT("RoyaltyTrackingSubText", "Please ensure a <a id=\"a\" href=\"{0}\">Notification of Release</> is submitted for tracking any future royalties."),
					FText::FromString(ReleaseUrl));
			}

			const UProjectPackagingSettings::FConfigurationInfo& ConfigurationInfo = UProjectPackagingSettings::ConfigurationInfo[(int)BuildConfig];
			if (BuildTargetInfo)
			{
				if (BuildTargetInfo->Type == EBuildTargetType::Client)
				{
					BuildCookRunParams += FString::Printf(TEXT(" -client -clientconfig=%s"), LexToString(ConfigurationInfo.Configuration));
				}
				else if (BuildTargetInfo->Type == EBuildTargetType::Server)
				{
					BuildCookRunParams += FString::Printf(TEXT(" -server -noclient -serverconfig=%s"), LexToString(ConfigurationInfo.Configuration));
				}
				else
				{
					BuildCookRunParams += FString::Printf(TEXT(" -clientconfig=%s"), LexToString(ConfigurationInfo.Configuration));
				}
			}

			if (ConfigurationInfo.Configuration == EBuildConfiguration::Shipping && !PackagingSettings->IncludeDebugFiles)
			{
				BuildCookRunParams += TEXT(" -nodebuginfo");
			}
		}
        else if (Mode == EPrepareContentMode::PrepareForDebugging)
         {
#if  PLATFORM_WINDOWS
             if (IniPlatformName == TEXT("IOS") || IniPlatformName == TEXT("TvOS"))
             {
				 bool bSupportSecondaryMac = false;
				 GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportSecondaryMac"), bSupportSecondaryMac, GEngineIni);
				 if (bSupportSecondaryMac)
				 {
					 FString OtherCommandLine = FString::Printf(TEXT("SetSecondaryRemoteMac -platform=%s -ProjectFilePath=\"%s\""), *IniPlatformName.ToString().ToLower(), *ProjectPath);
					 FTurnkeyEditorSupport::RunUAT(OtherCommandLine, PlatformInfo->DisplayName, ContentPrepDescription, ContentPrepTaskName, ContentPrepIcon, &AnalyticsParamArray);
				 }
				 else
				 {
					 FString CommandLine = FString::Printf(TEXT("WrangleContentForDebugging -platform=%s -ProjectFilePath=\"%s\""), *IniPlatformName.ToString().ToLower(), *ProjectPath);
					 FTurnkeyEditorSupport::RunUAT(CommandLine, PlatformInfo->DisplayName, ContentPrepDescription, ContentPrepTaskName, ContentPrepIcon, &AnalyticsParamArray);
				 }

				 return;
             }
#elif PLATFORM_MAC
		 if (IniPlatformName == TEXT("IOS") || IniPlatformName == TEXT("TvOS"))
		 {
			 FString CommandLine = FString::Printf(TEXT("WrangleContentForDebugging -platform=%s -ProjectFilePath=\"%s\""), *IniPlatformName.ToString().ToLower(), *ProjectPath);
			 FTurnkeyEditorSupport::RunUAT(CommandLine, PlatformInfo->DisplayName, ContentPrepDescription, ContentPrepTaskName, ContentPrepIcon, &AnalyticsParamArray);
			 return;
		 }
#endif
 		}
		else if (Mode == EPrepareContentMode::CookOnly)
		{
			ContentPrepDescription = LOCTEXT("CookingContentTaskName", "Cooking content");
			ContentPrepTaskName = LOCTEXT("CookingTaskName", "Cooking");
			ContentPrepIcon = FAppStyle::Get().GetBrush(TEXT("MainFrame.CookContent"));


			UCookerSettings const* CookerSettings = GetDefault<UCookerSettings>();
			if (CookerSettings->bIterativeCookingForFileCookContent)
			{
				BuildCookRunParams += TEXT(" -iterate");
			}

			BuildCookRunParams += TEXT(" -skipstage");
		}


		FString TurnkeyParams = FString::Printf(TEXT("-command=VerifySdk -platform=%s -UpdateIfNeeded %s"), *UBTPlatformString, *ITurnkeyIOModule::Get().GetUATParams());
		if (!ProjectPath.IsEmpty())
		{
			TurnkeyParams.Appendf(TEXT(" -project=\"%s\""), *ProjectPath);
		}

		FString CommandLine;
		if (!ProjectPath.IsEmpty())
		{
			CommandLine.Appendf(TEXT(" -ScriptsForProject=\"%s\" "), *ProjectPath);
		}
		CommandLine.Appendf(TEXT("Turnkey %s BuildCookRun %s"), *TurnkeyParams, *BuildCookRunParams);

		TFunction<void(FString, double)> ResultCallback;
		FTurnkeyEditorSupport::RunUAT(CommandLine, PlatformInfo->DisplayName, ContentPrepDescription, ContentPrepTaskName, ContentPrepIcon, &AnalyticsParamArray, ResultCallback, RoyaltySubText);
	}

	static FName GetPlatformForCustomBuild(ECustomBuildType Type, FString CustomBuildName)
	{
		return GetDefault<UPlatformsMenuSettings>()->GetPlatformForCustomBuild(Type, CustomBuildName);
	}

	static void SetPlatformForCustomBuild(ECustomBuildType Type, FString CustomBuildName, FName IniPlatformName)
	{
		UPlatformsMenuSettings* PlatformsSettings = GetMutableDefault<UPlatformsMenuSettings>();
		PlatformsSettings->SetPlatformForCustomBuild(Type, CustomBuildName, IniPlatformName);
		PlatformsSettings->SaveConfig();
	}

	static bool PlatformForCustomBuildIsChecked(ECustomBuildType Type, FString CustomBuildName, FName IniPlatformName)
	{
		const UPlatformsMenuSettings* PlatformsSettings = GetDefault<UPlatformsMenuSettings>();
		FName CheckedPlatform = PlatformsSettings->GetPlatformForCustomBuild(Type, CustomBuildName);
		return CheckedPlatform == IniPlatformName;
	}

	// Per-device custom builds show a list of devices rather than platforms. Since we know the platform associated with
	// each device, we set both at once to ensure the platform is set.
	static void SetDeviceIdForCustomBuild(ECustomBuildType Type, FString CustomBuildName, FName IniPlatformName, FString DeviceId)
	{
		UPlatformsMenuSettings* PlatformsSettings = GetMutableDefault<UPlatformsMenuSettings>();
		PlatformsSettings->SetPlatformForCustomBuild(Type, CustomBuildName, IniPlatformName);
		PlatformsSettings->SetDeviceIdForCustomBuild(Type, CustomBuildName, DeviceId);
		PlatformsSettings->SaveConfig();
	}

	static bool DeviceIdForCustomBuildIsChecked(ECustomBuildType Type, FString CustomBuildName, FString DeviceId)
	{
		const UPlatformsMenuSettings* PlatformsSettings = GetDefault<UPlatformsMenuSettings>();
		FString CheckedDeviceId = PlatformsSettings->GetDeviceIdForCustomBuild(Type, CustomBuildName);
		return CheckedDeviceId == DeviceId;
	}

	static void SetBuildConfigurationForCustomBuild(ECustomBuildType Type, FString CustomBuildName, EProjectPackagingBuildConfigurations BuildConfiguration)
	{
		UPlatformsMenuSettings* PlatformsSettings = GetMutableDefault<UPlatformsMenuSettings>();
		PlatformsSettings->SetBuildConfigurationForCustomBuild(Type, CustomBuildName, BuildConfiguration);
		PlatformsSettings->SaveConfig();
	}

	static bool BuildConfigurationForCustomBuildIsChecked(ECustomBuildType Type, FString CustomBuildName, EProjectPackagingBuildConfigurations BuildConfiguration)
	{
		const UPlatformsMenuSettings* PlatformsSettings = GetDefault<UPlatformsMenuSettings>();
		EProjectPackagingBuildConfigurations CheckedBuildConfiguration = PlatformsSettings->GetBuildConfigurationForCustomBuild(Type, CustomBuildName);
		return CheckedBuildConfiguration == BuildConfiguration;
	}

	static void SetBuildTargetForCustomBuild(ECustomBuildType Type, FString CustomBuildName, FString TargetName)
	{
		UPlatformsMenuSettings* PlatformsSettings = GetMutableDefault<UPlatformsMenuSettings>();
		PlatformsSettings->SetBuildTargetForCustomBuild(Type, CustomBuildName, TargetName);
		PlatformsSettings->SaveConfig();
	}

	static bool BuildTargetForCustomBuildIsChecked(ECustomBuildType Type, FString CustomBuildName, FString TargetName)
	{
		const UPlatformsMenuSettings* PlatformsSettings = GetDefault<UPlatformsMenuSettings>();
		FString CheckedTargetName = PlatformsSettings->GetBuildTargetForCustomBuild(Type, CustomBuildName);
		return CheckedTargetName == TargetName;
	}

	static bool CanExecuteCustomBuild(ECustomBuildType Type, FString BuildName, bool bNeedsDevice)
	{
		// Read the settings associated with this build
		const UPlatformsMenuSettings* MenuSettings = GetDefault<UPlatformsMenuSettings>();

		FName IniPlatformName = MenuSettings->GetPlatformForCustomBuild(Type, BuildName);

		if (GetTargetPlatformManager()->FindTargetPlatform(IniPlatformName.ToString()) == nullptr)
		{
			return false;
		}

		if (bNeedsDevice)
		{
			FString DeviceId = MenuSettings->GetDeviceIdForCustomBuild(Type, BuildName);

			if (DeviceId.IsEmpty())
			{
				return false;
			}
		}

		return true;
	}

	static FString GetCustomBuildCommandLine(FName IniPlatformName, EProjectPackagingBuildConfigurations BuildConfigSetting, const FString& BuildTargetName, const FString& DeviceId, const FProjectBuildSettings& Build)
	{
		const FString ProjectPath = GetProjectPathForTurnkey();

		FString CommandLine;
		CommandLine.Appendf(TEXT("Turnkey -command=ExecuteBuild -build=\"%s\" -platform=%s"), *Build.Name, *ConvertToUATPlatform(IniPlatformName.ToString()));
		if (!ProjectPath.IsEmpty())
		{
			CommandLine.Appendf(TEXT(" -project=\"%s\""), *ProjectPath);
		}
		if (!DeviceId.IsEmpty())
		{
			CommandLine.Appendf(TEXT(" -device=\"%s\""), *ConvertToUATDeviceId(DeviceId));
		}

		// pass the editor setting down to UAT in case they aren't saved to the ini's that UAT would read
		{
			UProjectPackagingSettings* PlatformPackagingSettings = FTurnkeySupportCallbacks::GetPackagingSettingsForPlatform(IniPlatformName);
			const UPlatformsMenuSettings* PlatformsSettings = GetDefault<UPlatformsMenuSettings>();
			bool bIsProjectBuildTarget = false;
			const FTargetInfo* BuildTargetInfo = PlatformsSettings->GetBuildTargetInfo(BuildTargetName, bIsProjectBuildTarget);

			CommandLine.Appendf(TEXT(" -overridetarget=%s"), *BuildTargetInfo->Name);

			// distribution builds can set shipping 
			EProjectPackagingBuildConfigurations BuildConfig = PlatformPackagingSettings->ForDistribution ?
				EProjectPackagingBuildConfigurations::PPBC_Shipping :
				BuildConfigSetting;

			// if PPBC_MAX is set, then the project default should be used instead of the per platform build config
			if (BuildConfig == EProjectPackagingBuildConfigurations::PPBC_MAX)
			{
				BuildConfig = PlatformPackagingSettings->BuildConfiguration;
			}
			const UProjectPackagingSettings::FConfigurationInfo& ConfigurationInfo = UProjectPackagingSettings::ConfigurationInfo[(int)BuildConfig];
			CommandLine.Appendf(TEXT(" -overrideconfiguration=%s"), LexToString(ConfigurationInfo.Configuration));


			// get the chosen cook flavor (texture format, etc)
			const PlatformInfo::FTargetPlatformInfo* PlatformInfo = PlatformInfo::FindPlatformInfo(PlatformsSettings->GetTargetFlavorForPlatform(IniPlatformName));
			if (PlatformInfo->IsFlavor())
			{
				CommandLine.Appendf(TEXT(" -overrideflavor=%s"), *PlatformInfo->PlatformFlavor.ToString());
			}
		}

		return CommandLine;
	}

	static void ExecuteCustomBuild(ECustomBuildType Type, FProjectBuildSettings Build)
	{
		// Read the settings associated with this build
		const UPlatformsMenuSettings* MenuSettings = GetDefault<UPlatformsMenuSettings>();

		FName IniPlatformName = MenuSettings->GetPlatformForCustomBuild(Type, Build.Name);
		FString DeviceId = MenuSettings->GetDeviceIdForCustomBuild(Type, Build.Name);
		FString BuildTargetName = MenuSettings->GetBuildTargetForCustomBuild(Type, Build.Name);
		EProjectPackagingBuildConfigurations BuildConfig = MenuSettings->GetBuildConfigurationForCustomBuild(Type, Build.Name);

		const PlatformInfo::FTargetPlatformInfo* PlatformInfo = PlatformInfo::FindPlatformInfo(GetDefault<UPlatformsMenuSettings>()->GetTargetFlavorForPlatform(IniPlatformName));
		const FString ProjectPath = GetProjectPathForTurnkey();

		// throw this on the command before the actual automation name (Turnkey, BuildCOokRun, etc) because they are global-to-UAT options, not options for a single command
		// and may not carry through when Turnkey passes along to the BCR command
		FString CommandLine = TEXT("-utf8output");
		if (!ProjectPath.IsEmpty())
		{
			CommandLine.Appendf(TEXT(" -ScriptsForProject=\"%s\" "), *ProjectPath);
		}
		CommandLine += GetCustomBuildCommandLine(IniPlatformName, BuildConfig, BuildTargetName, DeviceId, Build);

#if ALLOW_CONTROL_TO_COPY_COMMANDLINE
		bool bIsControlHeld = FSlateApplication::Get().GetModifierKeys().IsControlDown();
		if (bIsControlHeld)
		{
			CommandLine += TEXT(" -PrintOnly ");

			FString LogFilename, ReportFilename;
			CommandLine += GetLogAndReportCommandline(LogFilename, ReportFilename);

			FTurnkeyEditorSupport::RunUAT(CommandLine, PlatformInfo->DisplayName, LOCTEXT("Turnkey_GettingCommandLine", "Copying Commandline"), LOCTEXT("Turnkey_CustomTaskNameCmdLine", "CustomCommandLine"), nullptr /* TaskIcon */, nullptr /* OptionalAnalyticsParamArray */,
				[BuildName=Build.Name, ReportFilename](FString, double)
				{
					FString ReportContents;
					FFileHelper::LoadFileToString(ReportContents, *ReportFilename);
					UE_LOGF(LogTurnkeySupport, Log, "Custom command '%ls' gave commandline: %ls", *BuildName, *ReportContents);
					if (ReportContents.Len())
					{
						FPlatformApplicationMisc::ClipboardCopy(*ReportContents);
					}
				});
		}
		else
#endif
		{
			// append the stuff needed for running under editor
			FString EditorSpecificCommandLine = FString::Printf(TEXT(" %s"), GetUATCompilationFlags());
			if (FDerivedDataCacheInterface* DDC = GetDerivedDataCache())
			{
				EditorSpecificCommandLine += FString::Printf(TEXT(" -ddc=%s"), DDC->GetGraphName());
			}

			CommandLine += FString::Printf(TEXT(" -extraoptions=\"%s\""), *EditorSpecificCommandLine);

			// if there is a Broese option, execute it now before passing to UAT to get the editor dialog
			if (Build.BuildCookRunParams.Contains("{BrowseForDir}"))
			{
				// we reuse the already-presenbt StagingDirectory to save it, although it's more like "BuildOutpuDirectory" now 
				UPlatformsMenuSettings* PlatformsSettings = GetMutableDefault<UPlatformsMenuSettings>();
				FString OutFolderName;
				if (!FDesktopPlatformModule::Get()->OpenDirectoryDialog(FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr), LOCTEXT("PackageDirectoryDialogTitle", "Package project...").ToString(), PlatformsSettings->StagingDirectory.Path, OutFolderName))
				{
					return;
				}

				PlatformsSettings->StagingDirectory.Path = OutFolderName;
				PlatformsSettings->SaveConfig();

				CommandLine += FString::Printf(TEXT(" -outputdir=\"%s\""), *OutFolderName);
			}


			FTurnkeyEditorSupport::RunUAT(CommandLine, PlatformInfo->DisplayName, LOCTEXT("Turnkey_CustomTaskNameVerbose", "Executing Custom Build"), LOCTEXT("Turnkey_CustomTaskName", "Custom"), FAppStyle::GetBrush(TEXT("MainFrame.PackageProject")));
		}
	}

	static void SetPackageBuildConfiguration(EProjectPackagingBuildConfigurations BuildConfiguration)
	{
		UPlatformsMenuSettings* PlatformsSettings = GetMutableDefault<UPlatformsMenuSettings>();
		PlatformsSettings->SetPackageBuildConfiguration(BuildConfiguration);
		PlatformsSettings->SaveConfig();
	}

	static bool PackageBuildConfigurationIsChecked(EProjectPackagingBuildConfigurations BuildConfiguration)
	{
		EProjectPackagingBuildConfigurations CheckedBuildConfiguration = GetDefault<UPlatformsMenuSettings>()->GetPackageBuildConfiguration();
		return CheckedBuildConfiguration == BuildConfiguration;
	}
	
	static void SetActiveFlavor(const PlatformInfo::FTargetPlatformInfo* Info)
	{
		UPlatformsMenuSettings* PlatformsSettings = GetMutableDefault<UPlatformsMenuSettings>();
		PlatformsSettings->SetTargetFlavorForPlatform(Info->IniPlatformName, Info->Name);
		PlatformsSettings->SaveConfig();
	}

	static bool SetActiveFlavorIsChecked(const PlatformInfo::FTargetPlatformInfo* Info)
	{
		return GetDefault<UPlatformsMenuSettings>()->GetTargetFlavorForPlatform(Info->IniPlatformName) == Info->Name;
	}

	static void SetActiveArchitecture(FName IniPlatformName, FString Architecture)
	{
		UPlatformsMenuSettings* PlatformsSettings = GetMutableDefault<UPlatformsMenuSettings>();
		PlatformsSettings->SetArchitectureForPlatform(IniPlatformName, Architecture);
		PlatformsSettings->SaveConfig();
	}

	static bool SetActiveArchitectureIsChecked(FName IniPlatformName, FString Architecture)
	{
		return GetDefault<UPlatformsMenuSettings>()->GetArchitectureForPlatform(IniPlatformName) == Architecture;
	}

	static void SetPackageBuildTarget(FString TargetName)
	{
		UPlatformsMenuSettings* PlatformsSettings = GetMutableDefault<UPlatformsMenuSettings>();
		PlatformsSettings->PackageBuildTarget = MoveTemp(TargetName);
		PlatformsSettings->SaveConfig();
	}

	static bool PackageBuildTargetIsChecked(FString TargetName)
	{
		return GetDefault<UPlatformsMenuSettings>()->PackageBuildTarget == TargetName;
	}

	static void SetLaunchOnBuildConfiguration(EProjectPackagingBuildConfigurations BuildConfiguration)
	{
		UPlatformsMenuSettings* PlatformsSettings = GetMutableDefault<UPlatformsMenuSettings>();
		PlatformsSettings->SetLaunchOnBuildConfiguration(BuildConfiguration);
		PlatformsSettings->SaveConfig();
	}

	static bool LaunchOnBuildConfigurationIsChecked(EProjectPackagingBuildConfigurations BuildConfiguration)
	{
		EProjectPackagingBuildConfigurations CheckedBuildConfiguration = GetDefault<UPlatformsMenuSettings>()->GetLaunchOnBuildConfiguration();
		return CheckedBuildConfiguration == BuildConfiguration;
	}

	static void SetLaunchOnBuildTarget(FString TargetName)
	{
		UPlatformsMenuSettings* PlatformsSettings = GetMutableDefault<UPlatformsMenuSettings>();
		PlatformsSettings->LaunchOnTarget = MoveTemp(TargetName);
		PlatformsSettings->SaveConfig();
	}

	static bool LaunchOnBuildTargetIsChecked(FString TargetName)
	{
		return GetDefault<UPlatformsMenuSettings>()->GetLaunchOnTargetInfo()->Name == TargetName;
	}

	static void SetCookOnTheFly()
	{
		UCookerSettings* CookerSettings = GetMutableDefault<UCookerSettings>();

		CookerSettings->bCookOnTheFlyForLaunchOn = !CookerSettings->bCookOnTheFlyForLaunchOn;
		CookerSettings->Modify(true);

		// Update source control

		FString ConfigPath = FPaths::ConvertRelativePathToFull(CookerSettings->GetDefaultConfigFilename());

		if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*ConfigPath))
		{
			if (ISourceControlModule::Get().IsEnabled())
			{
				FText ErrorMessage;

				if (!SourceControlHelpers::CheckoutOrMarkForAdd(ConfigPath, FText::FromString(ConfigPath), NULL, ErrorMessage))
				{
					FNotificationInfo Info(ErrorMessage);
					Info.ExpireDuration = 3.0f;
					FSlateNotificationManager::Get().AddNotification(Info);
				}
			}
			else
			{
				if (!FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*ConfigPath, false))
				{
					FNotificationInfo Info(FText::Format(LOCTEXT("FailedToMakeWritable", "Could not make {0} writable."), FText::FromString(ConfigPath)));
					Info.ExpireDuration = 3.0f;
					FSlateNotificationManager::Get().AddNotification(Info);
				}
			}
		}

		// Save settings
		CookerSettings->UpdateSinglePropertyInConfigFile(CookerSettings->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UCookerSettings, bCookOnTheFlyForLaunchOn)), CookerSettings->GetDefaultConfigFilename());

	}

	static bool SetCookOnTheFlyIsChecked()
	{
		return GetDefault<UCookerSettings>()->bCookOnTheFlyForLaunchOn;
	}

	static FName GetCookPlatform()
	{
		return GetDefault<UPlatformsMenuSettings>()->CookPlatform;
	}

	static void SetCookPlatform(FName IniPlatformName)
	{
		UPlatformsMenuSettings* PlatformsSettings = GetMutableDefault<UPlatformsMenuSettings>();
		PlatformsSettings->CookPlatform = IniPlatformName;
		PlatformsSettings->SaveConfig();
	}

	static bool CookPlatformIsChecked(FName IniPlatformName)
	{
		return GetDefault<UPlatformsMenuSettings>()->CookPlatform == IniPlatformName;
	}

	static void SetCookBuildTarget(FString TargetName)
	{
		UPlatformsMenuSettings* PlatformsSettings = GetMutableDefault<UPlatformsMenuSettings>();
		PlatformsSettings->CookBuildTarget = MoveTemp(TargetName);
		PlatformsSettings->SaveConfig();
	}

	static bool CookBuildTargetIsChecked(FString TargetName)
	{
		return GetDefault<UPlatformsMenuSettings>()->CookBuildTarget == TargetName;
	}

	static bool CanCookContent()
	{
		return CanCookOrPackage(GetDefault<UPlatformsMenuSettings>()->CookPlatform, EPrepareContentMode::CookOnly);
	}

	static void CookContent()
	{
		const UPlatformsMenuSettings* PlatformsSettings = GetDefault<UPlatformsMenuSettings>();
		CookOrPackage(PlatformsSettings->CookPlatform, PlatformsSettings->CookBuildTarget, EPrepareContentMode::CookOnly);
	}

	static FName GetPackagePlatform()
	{
		return GetDefault<UPlatformsMenuSettings>()->PackagePlatform;
	}

	static void SetPackagePlatform(FName IniPlatformName)
	{
		UPlatformsMenuSettings* PlatformsSettings = GetMutableDefault<UPlatformsMenuSettings>();
		PlatformsSettings->PackagePlatform = IniPlatformName;
		PlatformsSettings->SaveConfig();
	}

	static bool PackagePlatformIsChecked(FName IniPlatformName)
	{
		return GetDefault<UPlatformsMenuSettings>()->PackagePlatform == IniPlatformName;
	}

	static bool CanPackageProject()
	{
		return CanCookOrPackage(GetDefault<UPlatformsMenuSettings>()->PackagePlatform, EPrepareContentMode::Package);
	}

	static void PackageProject()
	{
		const UPlatformsMenuSettings* PlatformsSettings = GetDefault<UPlatformsMenuSettings>();
		CookOrPackage(PlatformsSettings->PackagePlatform, PlatformsSettings->PackageBuildTarget, EPrepareContentMode::Package);
	}

	static FName GetPrepareForDebuggingPlatform()
	{
		return GetDefault<UPlatformsMenuSettings>()->PrepareForDebuggingPlatform;
	}

	static void SetPrepareForDebuggingPlatform(FName IniPlatformName)
	{
		UPlatformsMenuSettings* PlatformsSettings = GetMutableDefault<UPlatformsMenuSettings>();
		PlatformsSettings->PrepareForDebuggingPlatform = IniPlatformName;
		PlatformsSettings->SaveConfig();
	}

	static bool PrepareForDebuggingPlatformIsChecked(FName IniPlatformName)
	{
		return GetDefault<UPlatformsMenuSettings>()->PrepareForDebuggingPlatform == IniPlatformName;
	}

	static void SetPrepareForDebuggingBuildTarget(FString TargetName)
	{
		UPlatformsMenuSettings* PlatformsSettings = GetMutableDefault<UPlatformsMenuSettings>();
		PlatformsSettings->PrepareForDebuggingBuildTarget = MoveTemp(TargetName);
		PlatformsSettings->SaveConfig();
	}

	static bool PrepareForDebuggingBuildTargetIsChecked(FString TargetName)
	{
		return GetDefault<UPlatformsMenuSettings>()->PrepareForDebuggingBuildTarget == TargetName;
	}

	static bool CanPrepareForDebugging()
	{
		return CanCookOrPackage(GetDefault<UPlatformsMenuSettings>()->PrepareForDebuggingPlatform, EPrepareContentMode::PrepareForDebugging);
	}

	static void PrepareForDebugging()
	{
		const UPlatformsMenuSettings* PlatformsSettings = GetDefault<UPlatformsMenuSettings>();
		CookOrPackage(PlatformsSettings->PrepareForDebuggingPlatform, PlatformsSettings->PrepareForDebuggingBuildTarget, EPrepareContentMode::PrepareForDebugging);
	}

	static void SetRefreshPlatformStatus()
	{
		ITurnkeySupportModule::Get().UpdateSdkInfo();
	}

	static bool CanRefreshPlatformStatus()
	{
		return true;
	}
};

class FTurnkeySupportCommands : public TCommands<FTurnkeySupportCommands>
{
public:
	void RegisterCommands()
	{
		UI_COMMAND(PackagingSettings, "Packaging Settings...", "Opens the settings for project packaging", EUserInterfaceActionType::Button, FInputChord());
		ActionList->MapAction(PackagingSettings, FExecuteAction::CreateLambda([] {} ));

	}

	TSharedPtr< FUICommandInfo > PackagingSettings;

private:

	friend class TCommands<FTurnkeySupportCommands>;

	FTurnkeySupportCommands()
		: TCommands<FTurnkeySupportCommands>("TurnkeySupport", LOCTEXT("TurnkeySupport", "Turnkey and General Platform Options"), "MainFrame", FAppStyle::Get().GetStyleSetName())
	{

	}

public:
	/** List of all of the main frame commands */
	static TSharedRef<FUICommandList> ActionList;

};

TSharedRef<FUICommandList> FTurnkeySupportCommands::ActionList = MakeShareable(new FUICommandList);



static void ShowInstallationHelp(FName IniPlatformName)
{
// 	const ITargetPlatform* Platform = GetTargetPlatformManager()->FindTargetPlatform(IniPlatformName.ToString());
// 	bool bProjectHasCode;
// 	EBuildConfiguration Configuration;
// 	bool bRequiresAssetNativization;
// 	FString TutorialPath, DocumentationPath;
// 	FText LogMessage;
// 
// 	Platform->CheckRequirements(bProjectHasCode, Configuration, bRequiresAssetNativization, TutorialPath, DocumentationPath, LogMessage);

	FTurnkeyEditorSupport::ShowInstallationHelp(IniPlatformName, FDataDrivenPlatformInfoRegistry::GetPlatformInfo(IniPlatformName).SDKTutorial);
}

static void TurnkeyInstallSdk(FString IniPlatformName, bool bPreferFull, bool bForceInstall, FString DeviceId)
{
	FString OptionalOptions;
	if (bPreferFull)
	{
		OptionalOptions += TEXT(" -PreferFull");
	}
	if (bForceInstall)
	{
		OptionalOptions += DeviceId.Len() > 0 ? TEXT(" -ForceDeviceInstall") : TEXT(" -ForceSdkInstall");
	}
	if (DeviceId.Len() > 0)
	{
		OptionalOptions += FString::Printf(TEXT(" -Device=%s"), *DeviceId);
	}

	const FString ProjectPath = GetProjectPathForTurnkey();
	FString CommandLine;
	if (!ProjectPath.IsEmpty())
	{
		CommandLine.Appendf(TEXT(" -ScriptsForProject=\"%s\" "), *ProjectPath);
	}
	CommandLine.Appendf(TEXT("Turnkey -command=VerifySdk -UpdateIfNeeded -platform=%s %s %s -noturnkeyvariables -utf8output -WaitForUATMutex"), *IniPlatformName, *OptionalOptions, *ITurnkeyIOModule::Get().GetUATParams());

	FText TaskName = LOCTEXT("InstallingSdk", "Installing Sdk");
	FTurnkeyEditorSupport::RunUAT(CommandLine, FText::FromString(IniPlatformName), TaskName, TaskName, FAppStyle::GetBrush(TEXT("MainFrame.PackageProject")), nullptr,
		[IniPlatformName](FString, double)
	{
		AsyncTask(ENamedThreads::GameThread, [IniPlatformName]()
		{

			// read in env var changes
			// @todo turnkey move this and make it mac/linux aware
			FString TurnkeyEnvVarsFilename = FPaths::Combine(FPaths::EngineIntermediateDir(), TEXT("Turnkey/PostTurnkeyVariables.bat"));

			if (IFileManager::Get().FileExists(*TurnkeyEnvVarsFilename))
			{
				TArray<FString> Contents;
				if (FFileHelper::LoadFileToStringArray(Contents, *TurnkeyEnvVarsFilename))
				{
					for (const FString& Line : Contents)
					{
						if (Line.StartsWith(TEXT("set ")))
						{
							// split the line
							FString VariableLine = Line.Mid(4);
							int Equals;
							if (VariableLine.FindChar('=', Equals))
							{
								// set the key/value
								FString Key = VariableLine.Mid(0, Equals);
								FString Value = VariableLine.Mid(Equals + 1);

								FPlatformMisc::SetEnvironmentVar(*Key, *Value);

								UE_LOGF(LogTurnkeySupport, Log, "Turnkey setting env var: %ls = %ls", *Key, *Value);
							}
						}
					}
				}
			}

			// update the Sdk status
//			FDataDrivenPlatformInfoRegistry::UpdateSdkStatus();
			GetTargetPlatformManager()->UpdateAfterSDKInstall(*IniPlatformName);
#if WITH_ENGINE
			RenderUtilsInit();
#endif

			if (FDataDrivenPlatformInfoRegistry::HasCompiledSupportForPlatform(*IniPlatformName, FDataDrivenPlatformInfoRegistry::EPlatformNameType::Ini))
			{
				FTurnkeyEditorSupport::ShowRestartToast();
			}
			else
			{
				FTurnkeyEditorSupport::ShowRebuildToast();
			}
		});
	}
	);
}

static TMap<FName, FString> PerPlatformLastChosen;
static TMap<FName, FString> PerPlatformLastSimChosen;

static TAttribute<FText> MakeSdkStatusAttribute(FName IniPlatformName, TSharedPtr< ITargetDeviceProxy> DeviceProxy)
{
	FString DisplayString = DeviceProxy ? DeviceProxy->GetName() : IniPlatformName.ToString();
	FString DeviceId = DeviceProxy ? DeviceProxy->GetTargetDeviceId(NAME_None) : FString();

	return TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda([IniPlatformName, DisplayString, DeviceId]()
	{
		// get the status, or Unknown if it's not there
		ETurnkeyPlatformSdkStatus Status = DeviceId.Len() ? ITurnkeySupportModule::Get().GetSdkInfoForDeviceId(DeviceId).Status : ITurnkeySupportModule::Get().GetSdkInfo(IniPlatformName, false).Status;

		if (Status == ETurnkeyPlatformSdkStatus::Querying)
		{
			FFormatNamedArguments LabelArguments;
			LabelArguments.Add(TEXT("DisplayName"), FText::FromString(DisplayString));
			return FText::Format(LOCTEXT("SDKStatusLabel", "{DisplayName} (Querying...)"), LabelArguments);
		}
		return FText::FromString(DisplayString);
	}));
}

static FSlateIcon MakePlatformSdkIconAttribute(FName IniPlatformName, TSharedPtr< ITargetDeviceProxy> DeviceProxy)
{
	FString DeviceId = DeviceProxy ? DeviceProxy->GetTargetDeviceId(DeviceProxy->GetVanillaPlatformId(NAME_None)) : FString();

	// get the status, or Unknown if it's not there
	ETurnkeyPlatformSdkStatus Status = DeviceId.Len() ? ITurnkeySupportModule::Get().GetSdkInfoForDeviceId(DeviceId).Status : ITurnkeySupportModule::Get().GetSdkInfo(IniPlatformName, false).Status;
	bool bDeviceWarning = (DeviceId.Len() && ITurnkeySupportModule::Get().GetSdkInfoForDeviceId(DeviceId).DeviceStatus != ETurnkeyDeviceStatus::SoftwareValid);

	if (Status == ETurnkeyPlatformSdkStatus::OutOfDate || Status == ETurnkeyPlatformSdkStatus::NoSdk || bDeviceWarning)
	{
		return FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.Warning"));
	}
	else if (Status == ETurnkeyPlatformSdkStatus::Error)
	{
		return FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.Error"));
	}
	else if (Status == ETurnkeyPlatformSdkStatus::Unknown)
	{
		return FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.Help"));
	}
	else
	{
		const ITargetPlatform* Platform = GetTargetPlatformManager()->FindTargetPlatform(IniPlatformName);
		if ((DeviceProxy != nullptr) && (Platform != nullptr) && !DeviceProxy->GetConnectionType().IsEmpty() &&
			Platform->SupportsFeature(ETargetPlatformFeatures::SupportsMultipleConnectionTypes))
		{
			FSlateIcon Icon;
			if (DeviceProxy->GetConnectionType().Contains(TEXT("Network")))
			{
				Icon = FSlateIcon(FAppStyle::Get().GetStyleSetName(), *(TEXT("DeviceDetails.WIFI.") + IniPlatformName.ToString()));
			}
			else if (DeviceProxy->GetConnectionType().Contains(TEXT("Wifi")))
			{
				Icon = FSlateIcon(FAppStyle::Get().GetStyleSetName(), *(TEXT("DeviceDetails.WIFI.") + IniPlatformName.ToString()));
			}
			else if (DeviceProxy->GetConnectionType().Contains(TEXT("USB")))
			{
				Icon = FSlateIcon(FAppStyle::Get().GetStyleSetName(), *(TEXT("DeviceDetails.USB.") + IniPlatformName.ToString()));
			}
			else
			{
				UE_LOGF(LogTurnkeySupport, Warning, "Unknown ConnectionType:%ls", *DeviceProxy->GetConnectionType());
				// Default to platform icon
				Icon = FSlateIcon(FAppStyle::Get().GetStyleSetName(), FDataDrivenPlatformInfoRegistry::GetPlatformInfo(IniPlatformName).GetIconStyleName(EPlatformIconSize::Normal));
			}

			// Verify that the icon exists by checking it's not using the "DefaultBrush" icon resource
			const FSlateBrush *brush = FAppStyle::GetBrush(Icon.GetStyleName());
			if (brush != nullptr && (brush->GetResourceName() != FAppStyle::GetDefaultBrush()->GetResourceName()))
			{
				return Icon;
			}
		}

		// Default to platform icon
		return FSlateIcon(FAppStyle::Get().GetStyleSetName(), FDataDrivenPlatformInfoRegistry::GetPlatformInfo(IniPlatformName).GetIconStyleName(EPlatformIconSize::Normal));
	}
}

// Special fake UUID used to run iOS app natively on local Apple Silicon mac, see also FIOSDeviceHelper::LocalMacDummyUUID
static const FString LocalMacDummyUUID = TEXT("10CA18AC-10CA18AC10CA18AC");

static void FormatSdkInfo(const FString& PlatformOrDevice, const FTurnkeySdkInfo& SdkInfo, FText& OutInfo, FText& OutToolTip)
{
#if PLATFORM_MAC_ARM64
	if (PlatformOrDevice == TEXT("IOS@") + LocalMacDummyUUID)
	{
		OutInfo = LOCTEXT("SdkInfo_LocalMac", "Run native iOS app on local Mac");
		OutToolTip = LOCTEXT("SdkInfo_LocalMacToolTip", "Use this device to run native iOS app on local Mac. It is still built for iOS platform");
		return;
	}
#endif
	
	TArray<FText> Lines;

	for (TPair<FString, FTurnkeySdkInfo::Version> Pair : SdkInfo.SDKVersions)
	{
		const FString& Name = Pair.Key;
		const FString& Min = Pair.Value.Min;
		const FString& Max = Pair.Value.Max;
		const FString& Current = Pair.Value.Current;
		FFormatNamedArguments VersionArgs;
		VersionArgs.Add(TEXT("Name"), FText::FromString(Name));
		VersionArgs.Add(TEXT("Min"), FText::FromString(Min));
		VersionArgs.Add(TEXT("Max"), FText::FromString(Max));
		VersionArgs.Add(TEXT("Current"), FText::FromString(Current.Len() ? Current : FString(TEXT("--"))));

		if (Min == Max)
		{
			Lines.Add(FText::Format(LOCTEXT("SdkInfo_AllowedSDK_Single", "Allowed {Name} Version: {Min}"), VersionArgs));
		}
		else if (Min == TEXT(""))
		{
			Lines.Add(FText::Format(LOCTEXT("SdkInfo_AllowedSDK_MaxOnly", "Allowed {Name} Versions: Up to {Max}"), VersionArgs));
		}
		else if (Max == TEXT(""))
		{
			Lines.Add(FText::Format(LOCTEXT("SdkInfo_AllowedSDK_MinOnly", "Allowed {Name} Versions: {Min} and up"), VersionArgs));
		}
		else
		{
			Lines.Add(FText::Format(LOCTEXT("SdkInfo_AllowedSDK_Range", "Allowed {Name} Versions: {Min} through {Max}"), VersionArgs));
		}
		Lines.Add(FText::Format(LOCTEXT("SdkInfo_AllowedSDK_Current", "  Installed: {Current}"), VersionArgs));
	}

	FFormatOrderedArguments Args;
	Args.Add(SdkInfo.SdkErrorInformation);
	Args.Add(FText::FromString(PlatformOrDevice));

	if (!SdkInfo.SdkErrorInformation.IsEmpty())
	{
		Lines.Add(FText::Format(LOCTEXT("SdkInfo_Error", "Error Info:\n{0}"), Args));
	}

	// now make a single \n delimted text
	OutInfo = FText::Join(FText::FromString(TEXT("\n")), Lines);

	// make a tooltip
	if (PlatformOrDevice.Contains(TEXT("@")))
	{
		OutToolTip = FText::Format(LOCTEXT("SdkInfo_ToolTip", "Information returned from:\nRunUAT Turnkey -command=VerifySdk -device={1}"), Args);
	}
	else
	{
		OutToolTip = FText::Format(LOCTEXT("SdkInfo_ToolTipPlatform", "Information returned from:\nRunUAT Turnkey -command=VerifySdk -platform={1}"), Args);
	}
}


static bool HasBuildForDeviceOrNot(const TArray<FProjectBuildSettings> Builds, bool bLookForDeviceBuilds)
{
	for (const FProjectBuildSettings& Build : Builds)
	{
		const bool bHasDeviceEntry = Build.BuildCookRunParams.Contains(TEXT("{DeviceId}"));
		// if it has a device, and we are looking for device, or vice versa, return true
		if (bHasDeviceEntry == bLookForDeviceBuilds)
		{
			return true;
		}
	}

	return true;
}

static void TurnkeySetDeviceAutoSoftwareUpdates(FString PlatformName, FString DeviceId, bool bEnableAutoSoftwareUpdates)
{
	FString CommandLine;
	const FString ProjectPath = GetProjectPathForTurnkey();
	if (!ProjectPath.IsEmpty())
	{
		CommandLine.Appendf(TEXT(" -ScriptsForProject=\"%s\" "), *ProjectPath);
	}
	CommandLine.Appendf(TEXT("Turnkey -command=DeviceAutoSoftwareUpdates -device=%s -enable=%s"), *DeviceId, bEnableAutoSoftwareUpdates ? TEXT("true") : TEXT("false"));

	FText TaskName = LOCTEXT("SetDeviceAutoSoftwareUpdates", "Set device auto software updates");
	FTurnkeyEditorSupport::RunUAT(CommandLine, FText::FromString(PlatformName), TaskName, TaskName, FAppStyle::GetBrush(TEXT("MainFrame.PackageProject")), nullptr,
		[DeviceId](FString, double)
		{
			ITurnkeySupportModule::Get().UpdateSdkInfoForDevices(TArray<FString>{DeviceId});
		});	
}

// Returns the IniPlatformName of all platforms categorized into "supported", "unsupported", and "uncompiled" matching
// the different behaviors used by the Platforms menu. "Uncompiled" indicates there is no compiled-in support for that
// platform. "Unsupported" indicates the platform was disabled via the project settings.
static void GetTurnkeyCategorizedPlatforms(TArray<FName>* OutSupportedPlatforms, TArray<FName>* OutUnsupportedPlatforms, TArray<FName>* OutUncompiledPlatforms)
{
	if (OutSupportedPlatforms != nullptr)
	{
		OutSupportedPlatforms->Empty();
	}

	if (OutUnsupportedPlatforms != nullptr)
	{
		OutUnsupportedPlatforms->Empty();
	}

	if (OutUncompiledPlatforms != nullptr)
	{
		OutUncompiledPlatforms->Empty();
	}

	FProjectStatus ProjectStatus;
	bool bProjectStatusIsValid = IProjectManager::Get().QueryStatusForCurrentProject(ProjectStatus);

	for (const auto& Pair : FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos())
	{
		if (Pair.Value.bIsFakePlatform || Pair.Value.bEnabledForUse == false)
		{
			continue;
		}

		FName IniPlatformName = Pair.Key;
		if (FDataDrivenPlatformInfoRegistry::IsPlatformHiddenFromUI(IniPlatformName))
		{
			continue;
		}

		if (!FDataDrivenPlatformInfoRegistry::HasCompiledSupportForPlatform(
				IniPlatformName, FDataDrivenPlatformInfoRegistry::EPlatformNameType::Ini
			))
		{
			if (OutUncompiledPlatforms != nullptr)
			{
				OutUncompiledPlatforms->Add(IniPlatformName);
			}

			continue;
		}

		if (bProjectStatusIsValid && !ProjectStatus.IsTargetPlatformSupported(IniPlatformName))
		{
			if (OutUnsupportedPlatforms != nullptr)
			{
				OutUnsupportedPlatforms->Add(IniPlatformName);
			}

			continue;
		}

		if (OutSupportedPlatforms != nullptr)
		{
			OutSupportedPlatforms->Add(IniPlatformName);
		}
	}
}

// Shows a sub-menu for the platform allowing selection of the flavor or architecture where appropriate
static void MakeTurnkeyPerPlatformMenu(UToolMenu* Menu, FName IniPlatformName, TArray<const PlatformInfo::FTargetPlatformInfo*> ValidFlavors, TArray<FString> Architectures)
{
	if (ValidFlavors.Num() > 1)
	{
		FToolMenuSection& FlavorSection = Menu->AddSection("FlavorSelection", LOCTEXT("TurnkeySection_FlavorSelection", "Flavor Selection"));
					
		for (const PlatformInfo::FTargetPlatformInfo* Info : ValidFlavors)
		{
			FlavorSection.AddMenuEntry(
				NAME_None,
				Info->DisplayName,
				FText(),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::SetActiveFlavor, Info),
					FCanExecuteAction(),
					FIsActionChecked::CreateStatic(&FTurnkeySupportCallbacks::SetActiveFlavorIsChecked, Info)
				),
				EUserInterfaceActionType::RadioButton
			);
		}
	}

	if (Architectures.Num() > 1)
	{
		FToolMenuSection& ArchitectureSection = Menu->AddSection("ArchitectureSelection", LOCTEXT("TurnkeySection_ArchitectureSelection", "Architecture Selection"));

		ArchitectureSection.AddMenuEntry(
			NAME_None,
			LOCTEXT("DefaultArchitectureName", "Project Default"),
			FText(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::SetActiveArchitecture, IniPlatformName, FString()),
				FCanExecuteAction(),
				FIsActionChecked::CreateStatic(&FTurnkeySupportCallbacks::SetActiveArchitectureIsChecked, IniPlatformName, FString())
			),
			EUserInterfaceActionType::RadioButton
		);

		const ITargetPlatform* TargetPlatform = GetTargetPlatformManager()->FindTargetPlatform(IniPlatformName);

		for (const FString& Architecture : Architectures)
		{
			bool bIsHostArchitecture = (IniPlatformName == FPlatformProperties::IniPlatformName()) && (Architecture == TargetPlatform->GetHostArchitecture());

			FText ArchLabel = bIsHostArchitecture 
				? FText::Format(LOCTEXT("HostArchLabel", "{0} (this platform)"), FText::FromString(Architecture)) 
				: FText::FromString(Architecture);

			ArchitectureSection.AddMenuEntry(
				NAME_None,
				ArchLabel,
				FText(),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::SetActiveArchitecture, IniPlatformName, Architecture),
					FCanExecuteAction(),
					FIsActionChecked::CreateStatic(&FTurnkeySupportCallbacks::SetActiveArchitectureIsChecked, IniPlatformName, Architecture)
				),
				EUserInterfaceActionType::RadioButton
			);
		}
	}
}

static void MakeCookContentMenuItems(UToolMenu* Menu)
{
	FToolMenuSection& TopSection = Menu->AddSection("CookSection");
	{
		TopSection.AddMenuEntry(
			NAME_None,
			LOCTEXT("Turnkey_CookContent", "Cook Content"),
			LOCTEXT("TurnkeyTooltip_CookContent", "Cook this project for the selected Platform and target"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "MainFrame.CookContent"),
			FUIAction(
				FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::CookContent),
				FCanExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::CanCookContent)
			)
		);
	}

	FToolMenuSection& PlatformsSection = Menu->AddSection("CookPlatformsSection", LOCTEXT("TurnkeySection_CookPlatforms", "Cook For"));
	{
		PlatformsSection.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& PlatformsSection)
		{
			ITargetDeviceServicesModule* TargetDeviceServicesModule = static_cast<ITargetDeviceServicesModule*>(FModuleManager::Get().LoadModule(TEXT("TargetDeviceServices")));

			TArray<FName> SupportedPlatforms;
			GetTurnkeyCategorizedPlatforms(&SupportedPlatforms, nullptr, nullptr);

			// Always ensure a platform is selected to avoid confusion when the button is disabled. This occurs when
			// there is no saved setting such as on first launch, or if the saved platform is no longer supported due to
			// an SDK being uninstalled.
			FName SelectedPlatform = FTurnkeySupportCallbacks::GetCookPlatform();
			bool bUseNextPlatformAsDefault = !SupportedPlatforms.Contains(SelectedPlatform);

			// Use the host platform as the default selection as long as it's in the list and supported.
			if (bUseNextPlatformAsDefault && SupportedPlatforms.Contains(GHostIniPlatformName))
			{
				FTurnkeySupportCallbacks::SetCookPlatform(GHostIniPlatformName);
				bUseNextPlatformAsDefault = !FTurnkeySupportCallbacks::CanCookContent();
			}

			for (const FName& IniPlatformName : SupportedPlatforms)
			{
				const PlatformInfo::FTargetPlatformInfo* VanillaInfo = PlatformInfo::FindVanillaPlatformInfo(IniPlatformName);

				const ITargetPlatform* TargetPlatform = GetTargetPlatformManager()->FindTargetPlatform(IniPlatformName);

				// Flavor selection exists for Android to be able to package ASTC, DXT, ETC2, etc
				TArray<const PlatformInfo::FTargetPlatformInfo*> ValidFlavors;

				// Architecture selection exists for platforms targeting multiple CPU architectures, such as Windows on
				// x64 and ARM64.
				TArray<FString> Architectures;

				if (VanillaInfo != nullptr)
				{
					ValidFlavors = VanillaInfo->Flavors.FilterByPredicate([](const PlatformInfo::FTargetPlatformInfo* Target)
						{
							// Editor isn't a valid platform type that users can target
							// The Build Target will choose client or server, so no need to show them as well
							return Target->PlatformType != EBuildTargetType::Editor && Target->PlatformType != EBuildTargetType::Client && Target->PlatformType != EBuildTargetType::Server;
						});
				}

				if (TargetPlatform != nullptr)
				{
					TargetPlatform->GetPossibleArchitectures(Architectures);
				}
			
				// Platforms that have multiple flavors or architectures are displayed as a radio button that also pops
				// out a sub-menu to choose the flavor and/or architecture.
				if (ValidFlavors.Num() > 1 || Architectures.Num() > 1)
				{
					FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
						NAME_None,
						MakeSdkStatusAttribute(IniPlatformName, nullptr),
						FText::FromString(IniPlatformName.ToString()),
						FNewToolMenuDelegate::CreateStatic(&MakeTurnkeyPerPlatformMenu, IniPlatformName, ValidFlavors, Architectures),
						FUIAction(
							FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::SetCookPlatform, IniPlatformName),
							FCanExecuteAction(),
							FIsActionChecked::CreateStatic(&FTurnkeySupportCallbacks::CookPlatformIsChecked, IniPlatformName)
						),
						EUserInterfaceActionType::RadioButton,
						false,
						MakePlatformSdkIconAttribute(IniPlatformName, nullptr)
					);

					PlatformsSection.AddEntry(Entry);
				}
				else
				{
					FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
						NAME_None,
						MakeSdkStatusAttribute(IniPlatformName, nullptr),
						FText::FromString(IniPlatformName.ToString()),
						MakePlatformSdkIconAttribute(IniPlatformName, nullptr),
						FUIAction(
							FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::SetCookPlatform, IniPlatformName),
							FCanExecuteAction(),
							FIsActionChecked::CreateStatic(&FTurnkeySupportCallbacks::CookPlatformIsChecked, IniPlatformName)
						),
						EUserInterfaceActionType::RadioButton
					);

					PlatformsSection.AddEntry(Entry);
				}

				// If the host platform is not supported, choose the first supported platform. If there are none, the
				// last item in the list ends up selected.
				if (bUseNextPlatformAsDefault)
				{
					FTurnkeySupportCallbacks::SetCookPlatform(IniPlatformName);
					bUseNextPlatformAsDefault = !FTurnkeySupportCallbacks::CanCookContent();
				}
			}
		}));
	}

	UProjectPackagingSettings* AllPlatformPackagingSettings = GetMutableDefault<UProjectPackagingSettings>();

	// Collect build targets. Content-only projects use Engine targets
	FProjectStatus ProjectStatus;
	bool bHasCode = IProjectManager::Get().QueryStatusForCurrentProject(ProjectStatus) && ProjectStatus.bCodeBasedProject;

	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	TArray<FTargetInfo> Targets = bHasCode ? DesktopPlatform->GetTargetsForCurrentProject() : DesktopPlatform->GetTargetsForProject(FString());

	if (Targets.Num() > 0)
	{
		Targets.Sort([](const FTargetInfo& A, const FTargetInfo& B) { return A.Name < B.Name; });
		
		const TArray<FTargetInfo> ValidTargets = Targets.FilterByPredicate([](const FTargetInfo& Target)
			{
				return Target.Type == EBuildTargetType::Game || Target.Type == EBuildTargetType::Client || Target.Type == EBuildTargetType::Server;
			});

		if (ValidTargets.Num() > 1)
		{
			// Set BuildTarget to default to Game if it hasn't been set (or if it was set, but is no longer valid)
			FString BuildTarget = AllPlatformPackagingSettings->BuildTarget;
			if (BuildTarget.IsEmpty() ||
			   !ValidTargets.ContainsByPredicate([&BuildTarget](const FTargetInfo& Target) { return Target.Name == BuildTarget; } ))
			{
				TArray<FTargetInfo> GameTargets = ValidTargets.FilterByPredicate([](const FTargetInfo& Target)
					{ return Target.Type == EBuildTargetType::Game;	});
				// if there are no Game targets, look for a Client target
				if (GameTargets.Num() == 0)
				{
					GameTargets = ValidTargets.FilterByPredicate([](const FTargetInfo& Target)
						{ return Target.Type == EBuildTargetType::Client;	});
				}
				// if no clients, then just _anything_ 
				if (GameTargets.Num() == 0)
				{
					GameTargets = ValidTargets;
				}

				AllPlatformPackagingSettings->BuildTarget = GameTargets[0].Name;
				AllPlatformPackagingSettings->SaveConfig();
			}

			FToolMenuSection& TargetSection = Menu->AddSection("BuildTarget", LOCTEXT("TurnkeySection_BuildTarget", "Build Target"));

			// Using the project default setting is represented by an empty string
			TargetSection.AddMenuEntry(
				NAME_None,
				FText::Format(LOCTEXT("DefaultPackageTarget",  "Use Project Setting ({0})"), FText::FromString(AllPlatformPackagingSettings->BuildTarget)),
				FText::Format(LOCTEXT("DefaultPackageTargetTooltip", "Package the {0} target"), FText::FromString(AllPlatformPackagingSettings->BuildTarget)),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::SetCookBuildTarget, FString()),
					FCanExecuteAction(),
					FIsActionChecked::CreateStatic(&FTurnkeySupportCallbacks::CookBuildTargetIsChecked, FString())
					),
				EUserInterfaceActionType::RadioButton
			);

			for (const FTargetInfo& Target : ValidTargets)
			{
				TargetSection.AddMenuEntry(
					NAME_None,
					FText::FromString(Target.Name),
					FText::Format(LOCTEXT("PackageTargetName", "Package the '{0}' target."), FText::FromString(Target.Name)),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::SetCookBuildTarget, Target.Name),
						FCanExecuteAction(),
						FIsActionChecked::CreateStatic(&FTurnkeySupportCallbacks::CookBuildTargetIsChecked, Target.Name)
					),
					EUserInterfaceActionType::RadioButton
				);
			}
		}
	}
}

static void MakePackageProjectMenuItems(UToolMenu* Menu)
{
	FToolMenuSection& TopSection = Menu->AddSection("PackageSection");
	{
		TopSection.AddMenuEntry(
			"PackageProject",
			LOCTEXT("Turnkey_PackageProject", "Package Project"),
			LOCTEXT("TurnkeyTooltip_PackageProject", "Package this project and archive it to a user-selected directory. This can then be used to install and run."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "MainFrame.PackageProject"),
			FUIAction(
				FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::PackageProject),
				FCanExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::CanPackageProject)
			)
		);
	}

	FToolMenuSection& PlatformsSection = Menu->AddSection("PackagePlatformsSection", LOCTEXT("PackagePlatformsSection", "Package For"));
	{
		PlatformsSection.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& PlatformsSection)
		{
			ITargetDeviceServicesModule* TargetDeviceServicesModule = static_cast<ITargetDeviceServicesModule*>(FModuleManager::Get().LoadModule(TEXT("TargetDeviceServices")));

			TArray<FName> SupportedPlatforms;
			GetTurnkeyCategorizedPlatforms(&SupportedPlatforms, nullptr, nullptr);

			// Always ensure a platform is selected to avoid confusion when the button is disabled. This occurs when
			// there is no saved setting such as on first launch, or if the saved platform is no longer supported due to
			// an SDK being uninstalled.
			FName SelectedPlatform = FTurnkeySupportCallbacks::GetPackagePlatform();
			bool bUseNextPlatformAsDefault = !SupportedPlatforms.Contains(SelectedPlatform);

			// Use the host platform as the default selection as long as it's in the list and supported.
			if (bUseNextPlatformAsDefault && SupportedPlatforms.Contains(GHostIniPlatformName))
			{
				FTurnkeySupportCallbacks::SetPackagePlatform(GHostIniPlatformName);
				bUseNextPlatformAsDefault = !FTurnkeySupportCallbacks::CanPackageProject();
			}

			for (const FName& IniPlatformName : SupportedPlatforms)
			{
				const PlatformInfo::FTargetPlatformInfo* VanillaInfo = PlatformInfo::FindVanillaPlatformInfo(IniPlatformName);

				const ITargetPlatform* TargetPlatform = GetTargetPlatformManager()->FindTargetPlatform(IniPlatformName);

				// Flavor selection exists for Android to be able to package ASTC, DXT, ETC2, etc
				TArray<const PlatformInfo::FTargetPlatformInfo*> ValidFlavors;

				// Architecture selection exists for platforms targeting multiple CPU architectures, such as Windows on
				// x64 and ARM64.
				TArray<FString> Architectures;

				if (VanillaInfo != nullptr)
				{
					ValidFlavors = VanillaInfo->Flavors.FilterByPredicate([](const PlatformInfo::FTargetPlatformInfo* Target)
						{
							// Editor isn't a valid platform type that users can target
							// The Build Target will choose client or server, so no need to show them as well
							return Target->PlatformType != EBuildTargetType::Editor && Target->PlatformType != EBuildTargetType::Client && Target->PlatformType != EBuildTargetType::Server;
						});
				}

				if (TargetPlatform != nullptr)
				{
					TargetPlatform->GetPossibleArchitectures(Architectures);
				}
			
				// Platforms that have multiple flavors or architectures are displayed as a radio button that also pops
				// out a sub-menu to choose the flavor and/or architecture.
				if (ValidFlavors.Num() > 1 || Architectures.Num() > 1)
				{
					FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
						NAME_None,
						MakeSdkStatusAttribute(IniPlatformName, nullptr),
						FText::FromString(IniPlatformName.ToString()),
						FNewToolMenuDelegate::CreateStatic(&MakeTurnkeyPerPlatformMenu, IniPlatformName, ValidFlavors, Architectures),
						FUIAction(
							FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::SetPackagePlatform, IniPlatformName),
							FCanExecuteAction(),
							FIsActionChecked::CreateStatic(&FTurnkeySupportCallbacks::PackagePlatformIsChecked, IniPlatformName)
						),
						EUserInterfaceActionType::RadioButton,
						false,
						MakePlatformSdkIconAttribute(IniPlatformName, nullptr)
					);

					PlatformsSection.AddEntry(Entry);
				}
				else
				{
					FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
						NAME_None,
						MakeSdkStatusAttribute(IniPlatformName, nullptr),
						FText::FromString(IniPlatformName.ToString()),
						MakePlatformSdkIconAttribute(IniPlatformName, nullptr),
						FUIAction(
							FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::SetPackagePlatform, IniPlatformName),
							FCanExecuteAction(),
							FIsActionChecked::CreateStatic(&FTurnkeySupportCallbacks::PackagePlatformIsChecked, IniPlatformName)
						),
						EUserInterfaceActionType::RadioButton
					);

					PlatformsSection.AddEntry(Entry);
				}

				// If the host platform is not supported, choose the first supported platform. If there are none, the
				// last item in the list ends up selected.
				if (bUseNextPlatformAsDefault)
				{
					FTurnkeySupportCallbacks::SetPackagePlatform(IniPlatformName);
					bUseNextPlatformAsDefault = !FTurnkeySupportCallbacks::CanPackageProject();
				}
			}
		}));
	}

	UProjectPackagingSettings* AllPlatformPackagingSettings = GetMutableDefault<UProjectPackagingSettings>();

	FToolMenuSection& ConfigSection = Menu->AddSection("BuildConfig", LOCTEXT("TurnkeySection_BuildConfig", "Build Configuration"));
	{
		const UProjectPackagingSettings::FConfigurationInfo& DefaultConfigInfo = UProjectPackagingSettings::ConfigurationInfo[static_cast<int32>(AllPlatformPackagingSettings->BuildConfiguration)];
		ConfigSection.AddMenuEntry(
			NAME_None,
			FText::Format(LOCTEXT("DefaultConfiguration",  "Use Project Setting ({0})"), DefaultConfigInfo.Name),
			DefaultConfigInfo.ToolTip,
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::SetPackageBuildConfiguration, EProjectPackagingBuildConfigurations::PPBC_MAX),
				FCanExecuteAction(),
				FIsActionChecked::CreateStatic(&FTurnkeySupportCallbacks::PackageBuildConfigurationIsChecked, EProjectPackagingBuildConfigurations::PPBC_MAX)
			),
			EUserInterfaceActionType::RadioButton
		);

		EProjectType ProjectType = FTurnkeyEditorSupport::DoesProjectHaveCode() ? EProjectType::Code : EProjectType::Content;
		TArray<EProjectPackagingBuildConfigurations> PackagingConfigurations = UProjectPackagingSettings::GetValidPackageConfigurations();

		for (EProjectPackagingBuildConfigurations PackagingConfiguration : PackagingConfigurations)
		{
			const UProjectPackagingSettings::FConfigurationInfo& ConfigurationInfo = UProjectPackagingSettings::ConfigurationInfo[(int)PackagingConfiguration];
			if (FInstalledPlatformInfo::Get().IsValid(TOptional<EBuildTargetType>(), TOptional<FString>(), ConfigurationInfo.Configuration, ProjectType, EInstalledPlatformState::Downloaded))
			{
				ConfigSection.AddMenuEntry(
					NAME_None,
					ConfigurationInfo.Name,
					ConfigurationInfo.ToolTip,
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::SetPackageBuildConfiguration, PackagingConfiguration),
						FCanExecuteAction(),
						FIsActionChecked::CreateStatic(&FTurnkeySupportCallbacks::PackageBuildConfigurationIsChecked, PackagingConfiguration)
					),
					EUserInterfaceActionType::RadioButton
				);
			}
		}
	}

	// Collect build targets. Content-only projects use Engine targets
	FProjectStatus ProjectStatus;
	bool bHasCode = IProjectManager::Get().QueryStatusForCurrentProject(ProjectStatus) && ProjectStatus.bCodeBasedProject;

	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	TArray<FTargetInfo> Targets = bHasCode ? DesktopPlatform->GetTargetsForCurrentProject() : DesktopPlatform->GetTargetsForProject(FString());

	if (Targets.Num() > 0)
	{
		Targets.Sort([](const FTargetInfo& A, const FTargetInfo& B) { return A.Name < B.Name; });
		
		const TArray<FTargetInfo> ValidTargets = Targets.FilterByPredicate([](const FTargetInfo& Target)
			{
				return Target.Type == EBuildTargetType::Game || Target.Type == EBuildTargetType::Client || Target.Type == EBuildTargetType::Server;
			});

		if (ValidTargets.Num() > 1)
		{
			// Set BuildTarget to default to Game if it hasn't been set (or if it was set, but is no longer valid)
			FString BuildTarget = AllPlatformPackagingSettings->BuildTarget;
			if (BuildTarget.IsEmpty() ||
			   !ValidTargets.ContainsByPredicate([&BuildTarget](const FTargetInfo& Target) { return Target.Name == BuildTarget; } ))
			{
				TArray<FTargetInfo> GameTargets = ValidTargets.FilterByPredicate([](const FTargetInfo& Target)
					{ return Target.Type == EBuildTargetType::Game;	});
				// if there are no Game targets, look for a Client target
				if (GameTargets.Num() == 0)
				{
					GameTargets = ValidTargets.FilterByPredicate([](const FTargetInfo& Target)
						{ return Target.Type == EBuildTargetType::Client;	});
				}
				// if no clients, then just _anything_ 
				if (GameTargets.Num() == 0)
				{
					GameTargets = ValidTargets;
				}

				AllPlatformPackagingSettings->BuildTarget = GameTargets[0].Name;
				AllPlatformPackagingSettings->SaveConfig();
			}

			FToolMenuSection& TargetSection = Menu->AddSection("BuildTarget", LOCTEXT("TurnkeySection_BuildTarget", "Build Target"));

			// Using the project default setting is represented by an empty string
			TargetSection.AddMenuEntry(
				NAME_None,
				FText::Format(LOCTEXT("DefaultPackageTarget",  "Use Project Setting ({0})"), FText::FromString(AllPlatformPackagingSettings->BuildTarget)),
				FText::Format(LOCTEXT("DefaultPackageTargetTooltip", "Package the {0} target"), FText::FromString(AllPlatformPackagingSettings->BuildTarget)),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::SetPackageBuildTarget, FString()),
					FCanExecuteAction(),
					FIsActionChecked::CreateStatic(&FTurnkeySupportCallbacks::PackageBuildTargetIsChecked, FString())
					),
				EUserInterfaceActionType::RadioButton
			);

			for (const FTargetInfo& Target : ValidTargets)
			{
				TargetSection.AddMenuEntry(
					NAME_None,
					FText::FromString(Target.Name),
					FText::Format(LOCTEXT("PackageTargetName", "Package the '{0}' target."), FText::FromString(Target.Name)),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::SetPackageBuildTarget, Target.Name),
						FCanExecuteAction(),
						FIsActionChecked::CreateStatic(&FTurnkeySupportCallbacks::PackageBuildTargetIsChecked, Target.Name)
					),
					EUserInterfaceActionType::RadioButton
				);
			}
		}
	}
}

static void MakePrepareForDebuggingMenuItems(UToolMenu* Menu)
{
	FToolMenuSection& TopSection = Menu->AddSection("PrepareSection");
	{
		TopSection.AddMenuEntry(
			"PrepareForDebugging",
			LOCTEXT("Turnkey_PrepareForDebugging", "Prepare For Debugging"),
			LOCTEXT("Turnkey_PrepareForDebugging_Tooltip", "Create files to help streaming the debugging pipeline on some platforms"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Debug"),
			FUIAction(
				FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::PrepareForDebugging),
				FCanExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::CanPrepareForDebugging)
			)
		);
	}

	FToolMenuSection& PlatformsSection = Menu->AddSection("PlatformsSection", LOCTEXT("TurnkeySection_PrepareForDebuggingPlatforms", "Prepare For"));
	{
		PlatformsSection.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& PlatformsSection)
		{
			TArray<FName> SupportedPlatforms;
			GetTurnkeyCategorizedPlatforms(&SupportedPlatforms, nullptr, nullptr);

			// Always ensure a platform is selected to avoid confusion when the button is disabled. This occurs when
			// there is no saved setting such as on first launch, or if the saved platform is no longer supported due to
			// an SDK being uninstalled.
			FName SelectedPlatform = FTurnkeySupportCallbacks::GetPrepareForDebuggingPlatform();
			bool bUseNextPlatformAsDefault = !SupportedPlatforms.Contains(SelectedPlatform) || (SelectedPlatform != TEXT("IOS") && SelectedPlatform != TEXT("TvOS"));

			for (const FName& IniPlatformName : SupportedPlatforms)
			{
				if (IniPlatformName != TEXT("IOS") && IniPlatformName != TEXT("TvOS"))
				{
					continue;
				}

				// Create a radio button per-platform
				FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
					NAME_None,
					MakeSdkStatusAttribute(IniPlatformName, nullptr),
					FText::Format(LOCTEXT("TurnkeyTooltip_PrepareForDebuggingPlatform", "Expects a working remote toolchain and an IPA package with the same name as the project file in the Binaries/{0}/ folder."), FText::FromString(IniPlatformName.ToString().ToUpper())),
					MakePlatformSdkIconAttribute(IniPlatformName, nullptr),
					FUIAction(
						FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::SetPrepareForDebuggingPlatform, IniPlatformName),
						FCanExecuteAction(),
						FIsActionChecked::CreateStatic(&FTurnkeySupportCallbacks::PrepareForDebuggingPlatformIsChecked, IniPlatformName)
					),
					EUserInterfaceActionType::RadioButton
				);

				PlatformsSection.AddEntry(Entry);

				// Choose the first supported platform. If there are none, the last item in the list ends up selected.
				if (bUseNextPlatformAsDefault)
				{
					FTurnkeySupportCallbacks::SetPrepareForDebuggingPlatform(IniPlatformName);
					bUseNextPlatformAsDefault = !FTurnkeySupportCallbacks::CanPrepareForDebugging();
				}
			}
		}));
	}

	UProjectPackagingSettings* AllPlatformPackagingSettings = GetMutableDefault<UProjectPackagingSettings>();

	// Collect build targets. Content-only projects use Engine targets
	FProjectStatus ProjectStatus;
	bool bHasCode = IProjectManager::Get().QueryStatusForCurrentProject(ProjectStatus) && ProjectStatus.bCodeBasedProject;

	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	TArray<FTargetInfo> Targets = bHasCode ? DesktopPlatform->GetTargetsForCurrentProject() : DesktopPlatform->GetTargetsForProject(FString());

	if (Targets.Num() > 0)
	{
		Targets.Sort([](const FTargetInfo& A, const FTargetInfo& B) { return A.Name < B.Name; });
		
		const TArray<FTargetInfo> ValidTargets = Targets.FilterByPredicate([](const FTargetInfo& Target)
			{
				return Target.Type == EBuildTargetType::Game || Target.Type == EBuildTargetType::Client || Target.Type == EBuildTargetType::Server;
			});

		if (ValidTargets.Num() > 1)
		{
			// Set BuildTarget to default to Game if it hasn't been set (or if it was set, but is no longer valid)
			FString BuildTarget = AllPlatformPackagingSettings->BuildTarget;
			if (BuildTarget.IsEmpty() ||
			   !ValidTargets.ContainsByPredicate([&BuildTarget](const FTargetInfo& Target) { return Target.Name == BuildTarget; } ))
			{
				TArray<FTargetInfo> GameTargets = ValidTargets.FilterByPredicate([](const FTargetInfo& Target)
					{ return Target.Type == EBuildTargetType::Game;	});
				// if there are no Game targets, look for a Client target
				if (GameTargets.Num() == 0)
				{
					GameTargets = ValidTargets.FilterByPredicate([](const FTargetInfo& Target)
						{ return Target.Type == EBuildTargetType::Client;	});
				}
				// if no clients, then just _anything_ 
				if (GameTargets.Num() == 0)
				{
					GameTargets = ValidTargets;
				}

				AllPlatformPackagingSettings->BuildTarget = GameTargets[0].Name;
				AllPlatformPackagingSettings->SaveConfig();
			}

			FToolMenuSection& TargetSection = Menu->AddSection("BuildTarget", LOCTEXT("TurnkeySection_BuildTarget", "Build Target"));

			// Using the project default setting is represented by an empty string
			TargetSection.AddMenuEntry(
				NAME_None,
				FText::Format(LOCTEXT("DefaultPackageTarget",  "Use Project Setting ({0})"), FText::FromString(AllPlatformPackagingSettings->BuildTarget)),
				FText::Format(LOCTEXT("DefaultPackageTargetTooltip", "Package the {0} target"), FText::FromString(AllPlatformPackagingSettings->BuildTarget)),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::SetPrepareForDebuggingBuildTarget, FString()),
					FCanExecuteAction(),
					FIsActionChecked::CreateStatic(&FTurnkeySupportCallbacks::PrepareForDebuggingBuildTargetIsChecked, FString())
					),
				EUserInterfaceActionType::RadioButton
			);

			for (const FTargetInfo& Target : ValidTargets)
			{
				TargetSection.AddMenuEntry(
					NAME_None,
					FText::FromString(Target.Name),
					FText::Format(LOCTEXT("PackageTargetName", "Package the '{0}' target."), FText::FromString(Target.Name)),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::SetPrepareForDebuggingBuildTarget, Target.Name),
						FCanExecuteAction(),
						FIsActionChecked::CreateStatic(&FTurnkeySupportCallbacks::PrepareForDebuggingBuildTargetIsChecked, Target.Name)
					),
					EUserInterfaceActionType::RadioButton
				);
			}
		}
	}
}

static void MakePerCustomBuildMenu(UToolMenu* ToolMenu, ECustomBuildType Type, FProjectBuildSettings CustomBuild, FText ToolTip)
{
	const bool bHasDeviceEntry = CustomBuild.BuildCookRunParams.Contains(TEXT("{DeviceId}"));

	FToolMenuSection& CommandSection = ToolMenu->AddSection("CommandSection");
	{
		CommandSection.AddMenuEntry(
			"ExecuteCustomBuild",
			FText::FromString(CustomBuild.Name),
			ToolTip,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Advanced"),
			FUIAction(
				FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::ExecuteCustomBuild, Type, CustomBuild),
				FCanExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::CanExecuteCustomBuild, Type, CustomBuild.Name, bHasDeviceEntry)
			)
		);
	}

	// Custom builds either show a list of platforms or a list of devices
	if (bHasDeviceEntry)
	{
		FToolMenuSection& DevicesSection = ToolMenu->AddSection("DevicesSection", LOCTEXT("TurnkeySection_Device", "Device"));

		ITargetDeviceServicesModule* TargetDeviceServicesModule = static_cast<ITargetDeviceServicesModule*>(FModuleManager::Get().LoadModule(TEXT("TargetDeviceServices")));

		TArray<FName> SupportedPlatforms;
		GetTurnkeyCategorizedPlatforms(&SupportedPlatforms, nullptr, nullptr);

		// Always ensure a device is selected to avoid confusion when the button is disabled. This occurs when
		// there is no saved setting such as on first launch, or if the saved platform is no longer supported due to
		// an SDK being uninstalled.
		bool bUseNextDeviceAsDefault = !FTurnkeySupportCallbacks::CanExecuteCustomBuild(Type, CustomBuild.Name, bHasDeviceEntry);

		for (const FName& IniPlatformName : SupportedPlatforms)
		{
			if (CustomBuild.SpecificPlatforms.IsEmpty() || CustomBuild.SpecificPlatforms.Contains(IniPlatformName))
			{
				TArray<TSharedPtr<ITargetDeviceProxy>> DeviceProxies;
				TargetDeviceServicesModule->GetDeviceProxyManager()->GetAllProxies(IniPlatformName, DeviceProxies);

				for (const TSharedPtr<ITargetDeviceProxy>& Proxy : DeviceProxies)
				{
					FString DeviceName = Proxy->GetName();
					FString DeviceId = Proxy->GetTargetDeviceId(NAME_None);

					DevicesSection.AddMenuEntry(
						NAME_None,
						MakeSdkStatusAttribute(IniPlatformName, Proxy),
						FText::Format(LOCTEXT("TurnkeyTooltip_CustomBuildDevice", "{0} ({1})"), FText::FromString(DeviceName), FText::FromName(IniPlatformName)),
						MakePlatformSdkIconAttribute(IniPlatformName, Proxy),
						FUIAction(
							FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::SetDeviceIdForCustomBuild, Type, CustomBuild.Name, IniPlatformName, DeviceId),
							FCanExecuteAction(),
							FIsActionChecked::CreateStatic(&FTurnkeySupportCallbacks::DeviceIdForCustomBuildIsChecked, Type, CustomBuild.Name, DeviceId)
						),
						EUserInterfaceActionType::RadioButton
					);

					// Choose the first valid device as a default.
					if (bUseNextDeviceAsDefault)
					{
						FTurnkeySupportCallbacks::SetDeviceIdForCustomBuild(Type, CustomBuild.Name, IniPlatformName, DeviceId);
						bUseNextDeviceAsDefault = !FTurnkeySupportCallbacks::CanExecuteCustomBuild(Type, CustomBuild.Name, bHasDeviceEntry);
					}
				}
			}
		}
	}
	else
	{
		FToolMenuSection& PlatformsSection = ToolMenu->AddSection("PlatformsSection", LOCTEXT("TurnkeySection_Platform", "Platform"));
		{
			TArray<FName> SupportedPlatforms;
			GetTurnkeyCategorizedPlatforms(&SupportedPlatforms, nullptr, nullptr);

			// Always ensure a platform is selected to avoid confusion when the button is disabled. This occurs when
			// there is no saved setting such as on first launch, or if the saved platform is no longer supported due to
			// an SDK being uninstalled.
			FName SelectedPlatform = FTurnkeySupportCallbacks::GetPlatformForCustomBuild(Type, CustomBuild.Name);
			bool bUseNextPlatformAsDefault = !SupportedPlatforms.Contains(SelectedPlatform);

			// Use the host platform as the default selection as long as it's in the list.
			if (bUseNextPlatformAsDefault && SupportedPlatforms.Contains(GHostIniPlatformName) &&
				(CustomBuild.SpecificPlatforms.IsEmpty() || CustomBuild.SpecificPlatforms.Contains(GHostIniPlatformName)))
			{
				FTurnkeySupportCallbacks::SetPlatformForCustomBuild(Type, CustomBuild.Name, GHostIniPlatformName);
				bUseNextPlatformAsDefault = !FTurnkeySupportCallbacks::CanExecuteCustomBuild(Type, CustomBuild.Name, bHasDeviceEntry);
			}

			for (const FName& IniPlatformName : SupportedPlatforms)
			{
				if (CustomBuild.SpecificPlatforms.IsEmpty() || CustomBuild.SpecificPlatforms.Contains(IniPlatformName))
				{
					const PlatformInfo::FTargetPlatformInfo* VanillaInfo = PlatformInfo::FindVanillaPlatformInfo(IniPlatformName);

					const ITargetPlatform* TargetPlatform = GetTargetPlatformManager()->FindTargetPlatform(IniPlatformName);

					// Flavor selection exists for Android to be able to package ASTC, DXT, ETC2, etc
					TArray<const PlatformInfo::FTargetPlatformInfo*> ValidFlavors;

					// Architecture selection exists for platforms targeting multiple CPU architectures, such as Windows on
					// x64 and ARM64.
					TArray<FString> Architectures;

					if (VanillaInfo != nullptr)
					{
						ValidFlavors = VanillaInfo->Flavors.FilterByPredicate([](const PlatformInfo::FTargetPlatformInfo* Target)
							{
								// Editor isn't a valid platform type that users can target
								// The Build Target will choose client or server, so no need to show them as well
								return Target->PlatformType != EBuildTargetType::Editor && Target->PlatformType != EBuildTargetType::Client && Target->PlatformType != EBuildTargetType::Server;
							});
					}

					if (TargetPlatform != nullptr)
					{
						TargetPlatform->GetPossibleArchitectures(Architectures);
					}
				
					// Platforms that have multiple flavors or architectures are displayed as a radio button that also pops
					// out a sub-menu to choose the flavor and/or architecture.
					if (ValidFlavors.Num() > 1 || Architectures.Num() > 1)
					{
						FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
							NAME_None,
							MakeSdkStatusAttribute(IniPlatformName, nullptr),
							FText::FromString(IniPlatformName.ToString()),
							FNewToolMenuDelegate::CreateStatic(&MakeTurnkeyPerPlatformMenu, IniPlatformName, ValidFlavors, Architectures),
							FUIAction(
								FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::SetPlatformForCustomBuild, Type, CustomBuild.Name, IniPlatformName),
								FCanExecuteAction(),
								FIsActionChecked::CreateStatic(&FTurnkeySupportCallbacks::PlatformForCustomBuildIsChecked, Type, CustomBuild.Name, IniPlatformName)
							),
							EUserInterfaceActionType::RadioButton,
							false,
							MakePlatformSdkIconAttribute(IniPlatformName, nullptr)
						);

						PlatformsSection.AddEntry(Entry);
					}
					else
					{
						FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
							NAME_None,
							MakeSdkStatusAttribute(IniPlatformName, nullptr),
							FText::FromString(IniPlatformName.ToString()),
							MakePlatformSdkIconAttribute(IniPlatformName, nullptr),
							FUIAction(
								FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::SetPlatformForCustomBuild, Type, CustomBuild.Name, IniPlatformName),
								FCanExecuteAction(),
								FIsActionChecked::CreateStatic(&FTurnkeySupportCallbacks::PlatformForCustomBuildIsChecked, Type, CustomBuild.Name, IniPlatformName)
							),
							EUserInterfaceActionType::RadioButton
						);

						PlatformsSection.AddEntry(Entry);
					}

					// If the host platform is not supported, choose the first platform.
					if (bUseNextPlatformAsDefault)
					{
						FTurnkeySupportCallbacks::SetPlatformForCustomBuild(Type, CustomBuild.Name, IniPlatformName);
						bUseNextPlatformAsDefault = !FTurnkeySupportCallbacks::CanExecuteCustomBuild(Type, CustomBuild.Name, bHasDeviceEntry);
					}
				}
			}
		}
	}

	UProjectPackagingSettings* AllPlatformPackagingSettings = GetMutableDefault<UProjectPackagingSettings>();

	FToolMenuSection& ConfigSection = ToolMenu->AddSection("BuildConfig", LOCTEXT("TurnkeySection_BuildConfig", "Build Configuration"));
	{
		const UProjectPackagingSettings::FConfigurationInfo& DefaultConfigInfo = UProjectPackagingSettings::ConfigurationInfo[static_cast<int32>(AllPlatformPackagingSettings->BuildConfiguration)];
		ConfigSection.AddMenuEntry(
			NAME_None,
			FText::Format(LOCTEXT("DefaultConfiguration",  "Use Project Setting ({0})"), DefaultConfigInfo.Name),
			DefaultConfigInfo.ToolTip,
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::SetBuildConfigurationForCustomBuild, Type, CustomBuild.Name, EProjectPackagingBuildConfigurations::PPBC_MAX),
				FCanExecuteAction(),
				FIsActionChecked::CreateStatic(&FTurnkeySupportCallbacks::BuildConfigurationForCustomBuildIsChecked, Type, CustomBuild.Name, EProjectPackagingBuildConfigurations::PPBC_MAX)
			),
			EUserInterfaceActionType::RadioButton
		);

		EProjectType ProjectType = FTurnkeyEditorSupport::DoesProjectHaveCode() ? EProjectType::Code : EProjectType::Content;
		TArray<EProjectPackagingBuildConfigurations> PackagingConfigurations = UProjectPackagingSettings::GetValidPackageConfigurations();

		for (EProjectPackagingBuildConfigurations PackagingConfiguration : PackagingConfigurations)
		{
			const UProjectPackagingSettings::FConfigurationInfo& ConfigurationInfo = UProjectPackagingSettings::ConfigurationInfo[(int)PackagingConfiguration];
			if (FInstalledPlatformInfo::Get().IsValid(TOptional<EBuildTargetType>(), TOptional<FString>(), ConfigurationInfo.Configuration, ProjectType, EInstalledPlatformState::Downloaded))
			{
				ConfigSection.AddMenuEntry(
					NAME_None,
					ConfigurationInfo.Name,
					ConfigurationInfo.ToolTip,
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::SetBuildConfigurationForCustomBuild, Type, CustomBuild.Name, PackagingConfiguration),
						FCanExecuteAction(),
						FIsActionChecked::CreateStatic(&FTurnkeySupportCallbacks::BuildConfigurationForCustomBuildIsChecked, Type, CustomBuild.Name, PackagingConfiguration)
					),
					EUserInterfaceActionType::RadioButton
				);
			}
		}
	}

	// Collect build targets. Content-only projects use Engine targets
	FProjectStatus ProjectStatus;
	bool bHasCode = IProjectManager::Get().QueryStatusForCurrentProject(ProjectStatus) && ProjectStatus.bCodeBasedProject;

	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	TArray<FTargetInfo> Targets = bHasCode ? DesktopPlatform->GetTargetsForCurrentProject() : DesktopPlatform->GetTargetsForProject(FString());

	if (Targets.Num() > 0)
	{
		Targets.Sort([](const FTargetInfo& A, const FTargetInfo& B) { return A.Name < B.Name; });
		
		const TArray<FTargetInfo> ValidTargets = Targets.FilterByPredicate([](const FTargetInfo& Target)
			{
				return Target.Type == EBuildTargetType::Game || Target.Type == EBuildTargetType::Client || Target.Type == EBuildTargetType::Server;
			});

		if (ValidTargets.Num() > 1)
		{
			// Set BuildTarget to default to Game if it hasn't been set (or if it was set, but is no longer valid)
			FString BuildTarget = AllPlatformPackagingSettings->BuildTarget;
			if (BuildTarget.IsEmpty() ||
			   !ValidTargets.ContainsByPredicate([&BuildTarget](const FTargetInfo& Target) { return Target.Name == BuildTarget; } ))
			{
				TArray<FTargetInfo> GameTargets = ValidTargets.FilterByPredicate([](const FTargetInfo& Target)
					{ return Target.Type == EBuildTargetType::Game;	});
				// if there are no Game targets, look for a Client target
				if (GameTargets.Num() == 0)
				{
					GameTargets = ValidTargets.FilterByPredicate([](const FTargetInfo& Target)
						{ return Target.Type == EBuildTargetType::Client;	});
				}
				// if no clients, then just _anything_ 
				if (GameTargets.Num() == 0)
				{
					GameTargets = ValidTargets;
				}

				AllPlatformPackagingSettings->BuildTarget = GameTargets[0].Name;
				AllPlatformPackagingSettings->SaveConfig();
			}

			FToolMenuSection& TargetSection = ToolMenu->AddSection("BuildTarget", LOCTEXT("TurnkeySection_BuildTarget", "Build Target"));

			// Using the project default setting is represented by an empty string
			TargetSection.AddMenuEntry(
				NAME_None,
				FText::Format(LOCTEXT("DefaultPackageTarget",  "Use Project Setting ({0})"), FText::FromString(AllPlatformPackagingSettings->BuildTarget)),
				FText::Format(LOCTEXT("DefaultPackageTargetTooltip", "Package the {0} target"), FText::FromString(AllPlatformPackagingSettings->BuildTarget)),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::SetBuildTargetForCustomBuild, Type, CustomBuild.Name, FString()),
					FCanExecuteAction(),
					FIsActionChecked::CreateStatic(&FTurnkeySupportCallbacks::BuildTargetForCustomBuildIsChecked, Type, CustomBuild.Name, FString())
					),
				EUserInterfaceActionType::RadioButton
			);

			for (const FTargetInfo& Target : ValidTargets)
			{
				TargetSection.AddMenuEntry(
					NAME_None,
					FText::FromString(Target.Name),
					FText::Format(LOCTEXT("PackageTargetName", "Package the '{0}' target."), FText::FromString(Target.Name)),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::SetBuildTargetForCustomBuild, Type, CustomBuild.Name, Target.Name),
						FCanExecuteAction(),
						FIsActionChecked::CreateStatic(&FTurnkeySupportCallbacks::BuildTargetForCustomBuildIsChecked, Type, CustomBuild.Name, Target.Name)
					),
					EUserInterfaceActionType::RadioButton
				);
			}
		}
	}
}

static void MakeCustomBuildsMenuItems(UToolMenu* Menu)
{
	UProjectPackagingSettings* AllPlatformPackagingSettings = GetMutableDefault<UProjectPackagingSettings>();

	// Items from EngineCustomBuilds and ProjectCustomBuilds are shown in separate sections. Since they are separate
	// arrays, it's possible they both contain an entry with the same name.
	FToolMenuSection& EngineSection = Menu->AddSection("EngineCustomBuilds", LOCTEXT("TurnkeySection_EngineCustomBuilds", "Engine Custom Builds"));

	for (FProjectBuildSettings& CustomBuild : AllPlatformPackagingSettings->EngineCustomBuilds)
	{
#if ALLOW_CONTROL_TO_COPY_COMMANDLINE
		FText ToolTipFormat = LOCTEXT("TurnkeyTooltip_EngineCustomBuild_WithCopy", "Execute the '{BuildName}' command.\n{BuildHelp}\n--\nHold Control to copy the final underlying BuildCookRun commandline to the clipboard.");
#else
		FText ToolTipFormat = LOCTEXT("TurnkeyTooltip_EngineCustomBuild", "Execute the '{BuildName}' command.\n{BuildHelp}");
#endif

		FFormatNamedArguments Args;
		Args.Add(TEXT("BuildName"), FText::FromString(CustomBuild.Name));
		Args.Add(TEXT("BuildHelp"), FText::FromString(CustomBuild.HelpText));

		FText ToolTip = FText::Format(ToolTipFormat, Args);

		EngineSection.AddSubMenu(
			NAME_None,
			FText::FromString(CustomBuild.Name),
			ToolTip,
			FNewToolMenuDelegate::CreateStatic(&MakePerCustomBuildMenu, ECustomBuildType::EngineCustomBuilds, CustomBuild, ToolTip),
			false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Advanced"),
			true
		);
	}

	FToolMenuSection& ProjectSection = Menu->AddSection("ProjectCustomBuilds", LOCTEXT("TurnkeySection_ProjectCustomBuilds", "Project Custom Builds"));

	for (FProjectBuildSettings& CustomBuild : AllPlatformPackagingSettings->ProjectCustomBuilds)
	{
#if ALLOW_CONTROL_TO_COPY_COMMANDLINE
		FText ToolTipFormat = LOCTEXT("TurnkeyTooltip_ProjectCustomBuild_WithCopy", "Execute this project's custom '{BuildName}' command.\n{BuildHelp}\n--\nHold Control to copy the final BuildCookRun commandline to the clipboard.\nThis custom command comes from Project Settings.");
#else
		FText ToolTipFormat = LOCTEXT("TurnkeyTooltip_ProjectCustomBuild", "Execute this project's custom '{BuildName}' command.\n{BuildHelp}\n--\nThis custom command comes from Project Settings.");
#endif

		FFormatNamedArguments Args;
		Args.Add(TEXT("BuildName"), FText::FromString(CustomBuild.Name));
		Args.Add(TEXT("BuildHelp"), FText::FromString(CustomBuild.HelpText));

		FText ToolTip = FText::Format(ToolTipFormat, Args);

		ProjectSection.AddSubMenu(
			NAME_None,
			FText::FromString(CustomBuild.Name),
			ToolTip,
			FNewToolMenuDelegate::CreateStatic(&MakePerCustomBuildMenu, ECustomBuildType::ProjectCustomBuilds, CustomBuild, ToolTip),
			false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Advanced"),
			true
		);
	}
}

static void MakeSdkManagementPerPlatformPlatformMenu(UToolMenu* ToolMenu, FName IniPlatformName, ITargetDeviceServicesModule* TargetDeviceServicesModule)
{
	const FDataDrivenPlatformInfo& DDPI = FDataDrivenPlatformInfoRegistry::GetPlatformInfo(IniPlatformName);
	FString UBTPlatformString = DDPI.UBTPlatformString;

	const PlatformInfo::FTargetPlatformInfo* VanillaInfo = PlatformInfo::FindVanillaPlatformInfo(IniPlatformName);

	const ITargetPlatform* TargetPlatform = GetTargetPlatformManager()->FindTargetPlatform(IniPlatformName);

	if (VanillaInfo != nullptr)
	{
		UProjectPackagingSettings* PackagingSettings = FTurnkeySupportCallbacks::GetPackagingSettingsForPlatform(IniPlatformName);

		UProjectPackagingSettings* AllPlatformPackagingSettings = GetMutableDefault<UProjectPackagingSettings>();

		FToolMenuSection& DevicesSection = ToolMenu->AddSection("AllDevices", LOCTEXT("TurnkeySection_AllDevices", "All Devices"));

		TArray<TSharedPtr<ITargetDeviceProxy>> DeviceProxies;
		TargetDeviceServicesModule->GetDeviceProxyManager()->GetAllProxies(IniPlatformName, DeviceProxies);

		for (const TSharedPtr<ITargetDeviceProxy>& Proxy : DeviceProxies)
		{
			FString DeviceName = Proxy->GetName();
			FString DeviceId = Proxy->GetTargetDeviceId(NAME_None);
			DevicesSection.AddSubMenu(
				NAME_None,
				MakeSdkStatusAttribute(IniPlatformName, Proxy),
				FText(),
				FNewToolMenuDelegate::CreateLambda([IniPlatformName, DeviceName, DeviceId, PackagingSettings](UToolMenu* SubToolMenu)
				{
					FTurnkeySdkInfo SdkInfo = ITurnkeySupportModule::Get().GetSdkInfoForDeviceId(DeviceId);
					FText SdkText, SdkToolTip;
					FormatSdkInfo(DeviceId, SdkInfo, SdkText, SdkToolTip);

					FToolMenuSection& Section = SubToolMenu->AddSection("DeviceSdkInfo", LOCTEXT("TurnkeySection_DeviceSdkInfo", "Sdk Info"));
					Section.AddEntry(
						FToolMenuEntry::InitWidget(
							NAME_None,
							SNew(STextBlock)
							.ColorAndOpacity(FSlateColor::UseSubduedForeground())
							.Text(SdkText)
							.ToolTip(SNew(SToolTip).Text(SdkToolTip)),
							FText::GetEmpty()
						)
					);

					ETurnkeyDeviceAutoSoftwareUpdateMode DeviceAutoUpdateStatus = SdkInfo.DeviceAutoSoftwareUpdates;
					if (DeviceAutoUpdateStatus != ETurnkeyDeviceAutoSoftwareUpdateMode::Unknown)
					{

						Section.AddMenuEntry(
							NAME_None,
							LOCTEXT("Turnkey_EnableAutoSoftwareUpdates", "Enable auto software updates"),
							LOCTEXT("Turnkey_EnableAutoSoftwareUpdatesDescription", "This device will automatically update its software when new versions are released"),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateStatic(TurnkeySetDeviceAutoSoftwareUpdates, IniPlatformName.ToString(), DeviceId, DeviceAutoUpdateStatus != ETurnkeyDeviceAutoSoftwareUpdateMode::Enabled),
								FCanExecuteAction(),
								FIsActionChecked::CreateLambda([DeviceAutoUpdateStatus]()
						{
							return DeviceAutoUpdateStatus == ETurnkeyDeviceAutoSoftwareUpdateMode::Enabled;
						})
							),
							EUserInterfaceActionType::ToggleButton
							);
					}

					if (SdkInfo.DeviceStatus == ETurnkeyDeviceStatus::SoftwareValid)
					{
						Section.AddMenuEntry(
							NAME_None,
							LOCTEXT("Turnkey_ForceRepairDevice", "Force Update Device"),
							LOCTEXT("TurnkeyTooltip_ForceRepairDevice", "Force repairing anything on the device needed (update firmware, etc). Will perform all steps possible, even if not needed."),
							FSlateIcon(),
							FExecuteAction::CreateStatic(TurnkeyInstallSdk, IniPlatformName.ToString(), false, true, DeviceId)
						);
					}
					else
					{
						Section.AddMenuEntry(
							NAME_None,
							LOCTEXT("Turnkey_RepairDevice", "Update Device"),
							LOCTEXT("TurnkeyTooltip_RepairDevice", "Perform any fixup that may be needed on this device. If up to date already, nothing will be done."),
							FSlateIcon(),
							FExecuteAction::CreateStatic(TurnkeyInstallSdk, IniPlatformName.ToString(), false, false, DeviceId)
						);
					}

				}),
				false, MakePlatformSdkIconAttribute(IniPlatformName, Proxy)
			);
		}
	}

	FToolMenuSection& SdkSection = ToolMenu->AddSection("SdkManagement", LOCTEXT("TurnkeySection_Sdks", "Sdk Managment"));

	const FTurnkeySdkInfo& SdkInfo = ITurnkeySupportModule::Get().GetSdkInfo(IniPlatformName, true);
	FText SdkText, SdkToolTip;
	FormatSdkInfo(IniPlatformName.ToString(), SdkInfo, SdkText, SdkToolTip);

	SdkSection.AddEntry(
		FToolMenuEntry::InitWidget(
			NAME_None,
			SNew(SBox)
			.Padding(FMargin(16.0f, 3.0f))
			[
				SNew(STextBlock)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Text(SdkText)
				.ToolTip(SNew(SToolTip).Text(SdkToolTip))
			],
			FText::GetEmpty()
		)
	);

	FString NoDevice;
	if (SdkInfo.bCanInstallFullSdk || SdkInfo.bCanInstallAutoSdk)
	{
		if (SdkInfo.Status == ETurnkeyPlatformSdkStatus::OutOfDate || (SdkInfo.Status == ETurnkeyPlatformSdkStatus::Valid && SdkInfo.bHasBestSdk == false))
		{
			SdkSection.AddMenuEntry(
				NAME_None,
				LOCTEXT("Turnkey_UpdateSdkMinimal", "Update Sdk"),
				LOCTEXT("TurnkeyTooltip_UpdateSdkMinimal", "Attempt to update an Sdk, as hosted by your studio. Will attempt to install a minimal Sdk (useful for building/running only)"),
				FSlateIcon(),
				FExecuteAction::CreateStatic(TurnkeyInstallSdk, IniPlatformName.ToString(), false, false, NoDevice)
			);

			if (SdkInfo.bCanInstallFullSdk && SdkInfo.bCanInstallAutoSdk)
			{
				SdkSection.AddMenuEntry(
					NAME_None,
					LOCTEXT("Turnkey_UpdateSdkFull", "Update Sdk (Full Platform Installer)"),
					LOCTEXT("TurnkeyTooltip_UpdateSdkFull", "Attempt to update an Sdk, as hosted by your studio. Will attempt to install a full Sdk (useful profiling or other use cases)"),
					FSlateIcon(),
					FExecuteAction::CreateStatic(TurnkeyInstallSdk, IniPlatformName.ToString(), true, false, NoDevice)
				);
			}
		}
		else if (SdkInfo.Status == ETurnkeyPlatformSdkStatus::Valid)
		{
			SdkSection.AddMenuEntry(
				NAME_None,
				LOCTEXT("Turnkey_ForceSdkMinimal", "Force Reinstall Sdk"),
				LOCTEXT("TurnkeyTooltip_ForceSdkMinimal", "Attempt to force re-install an Sdk, as hosted by your studio. Will attempt to install a minimal Sdk (useful for building/running only)"),
				FSlateIcon(),
				FExecuteAction::CreateStatic(TurnkeyInstallSdk, IniPlatformName.ToString(), false, true, NoDevice)
			);

			if (SdkInfo.bCanInstallFullSdk && SdkInfo.bCanInstallAutoSdk)
			{
				SdkSection.AddMenuEntry(
					NAME_None,
					LOCTEXT("Turnkey_ForceSdkFull", "Force Reinstall (Full Platform Installer)"),
					LOCTEXT("TurnkeyTooltip_ForceSdkForce", "Attempt to force re-install an Sdk, as hosted by your studio. Will attempt to install a full Sdk (useful profiling or other use cases)"),
					FSlateIcon(),
					FExecuteAction::CreateStatic(TurnkeyInstallSdk, IniPlatformName.ToString(), true, true, NoDevice)
				);
			}
		}
		else
		{
			SdkSection.AddMenuEntry(
				NAME_None,
				LOCTEXT("Turnkey_InstallSdkMinimal", "Install Sdk"),
				LOCTEXT("TurnkeyTooltip_InstallSdkMinimal", "Attempt to install an Sdk, as hosted by your studio. Will attempt to install a minimal Sdk (useful for building/running only)"),
				FSlateIcon(),
				FExecuteAction::CreateStatic(TurnkeyInstallSdk, IniPlatformName.ToString(), false, false, NoDevice)
			);

			SdkSection.AddMenuEntry(
				NAME_None,
				LOCTEXT("Turnkey_InstallSdkFull", "Install Sdk (Full Platform Installer)"),
				LOCTEXT("TurnkeyTooltip_InstallSdkFull", "Attempt to install an Sdk, as hosted by your studio. Will attempt to install a full Sdk (useful profiling or other use cases)"),
				FSlateIcon(),
				FExecuteAction::CreateStatic(TurnkeyInstallSdk, IniPlatformName.ToString(), true, false, NoDevice)
			);
		}
	}

#if 0 // @todo turnkey enable and always show documentation for installation help when we have URLs in place
	// Link to documentation
	SdkSection.AddMenuEntry(
		NAME_None,
		LOCTEXT("Turnkey_ShowDocumentation", "Installation Help..."),
		LOCTEXT("TurnkeyTooltip_ShowDocumentation", "Show documentation with help installing the SDK for this platform"),
		FSlateIcon(),
		FExecuteAction::CreateStatic(ShowInstallationHelp, IniPlatformName)
	);
#endif
}


static void MakeSdkManagementMenuItems(UToolMenu* SdkManagementMenu)
{
	SdkManagementMenu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([](UToolMenu* Menu)
	{
		FToolMenuSection& SupportedPlatformsSection = Menu->AddSection("SupportedPlatforms", LOCTEXT("TurnkeySection_SupportedPlatforms", "Platforms"));

		ITargetDeviceServicesModule* TargetDeviceServicesModule = static_cast<ITargetDeviceServicesModule*>(FModuleManager::Get().LoadModule(TEXT("TargetDeviceServices")));

		TArray<FName> SupportedPlatforms;
		TArray<FName> UnsupportedPlatforms;
		TArray<FName> UncompiledPlatforms;
		GetTurnkeyCategorizedPlatforms(&SupportedPlatforms, &UnsupportedPlatforms, &UncompiledPlatforms);

		for (const FName& IniPlatformName : SupportedPlatforms)
		{
			SupportedPlatformsSection.AddSubMenu(
				NAME_None,
				MakeSdkStatusAttribute(IniPlatformName, nullptr),
				FText::FromString(IniPlatformName.ToString()),
				FNewToolMenuDelegate::CreateStatic(&MakeSdkManagementPerPlatformPlatformMenu, IniPlatformName, TargetDeviceServicesModule),
				false,
				MakePlatformSdkIconAttribute(IniPlatformName, nullptr),
				true
			);
		}

		if (UnsupportedPlatforms.Num() != 0)
		{
			FToolMenuSection& UnsupportedPlatformsSection = Menu->AddSection("UnsupportedPlatforms", LOCTEXT("Turnkey_UnsupportedPlatforms", "Unsupported Platforms"));

			for (const FName& IniPlatformName : UnsupportedPlatforms)
			{
				UnsupportedPlatformsSection.AddSubMenu(
					NAME_None,
					MakeSdkStatusAttribute(IniPlatformName, nullptr),
					FText::FromString(IniPlatformName.ToString()),
					FNewToolMenuDelegate::CreateStatic(&MakeSdkManagementPerPlatformPlatformMenu, IniPlatformName, TargetDeviceServicesModule),
					false,
					MakePlatformSdkIconAttribute(IniPlatformName, nullptr),
					true
				);
			}
		}

		if (UncompiledPlatforms.Num() != 0)
		{
			FToolMenuSection& UncompiledPlatformsSection = Menu->AddSection("UncompiledPlatforms", LOCTEXT("Turnkey_UncompiledPlatforms", "Platforms With No Compiled Support"));

			for (const FName& IniPlatformName : UncompiledPlatforms)
			{
				UncompiledPlatformsSection.AddSubMenu(
					NAME_None,
					MakeSdkStatusAttribute(IniPlatformName, nullptr),
					FText::FromString(IniPlatformName.ToString()),
					FNewToolMenuDelegate::CreateStatic(&MakeSdkManagementPerPlatformPlatformMenu, IniPlatformName, TargetDeviceServicesModule),
					false,
					MakePlatformSdkIconAttribute(IniPlatformName, nullptr),
					true
				);
			}
		}

		FToolMenuSection& RefreshSection = Menu->AddSection("RefreshSection");

		RefreshSection.AddSeparator(NAME_None);
		
		RefreshSection.AddMenuEntry(
			NAME_None,
			LOCTEXT("RefreshPlatformStatus", "Refresh platform status"),
			LOCTEXT("RefreshPlatformStatusDescription", "Update all platforms and device information"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "SourceControl.Actions.Refresh"),
			FUIAction(
				FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::SetRefreshPlatformStatus),
				FCanExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::CanRefreshPlatformStatus)
			)
		);
	}));
}

// Launch On

bool CanLaunchOnDevice(const FString& DeviceName)
{
	static TWeakPtr<ITargetDeviceProxyManager> DeviceProxyManagerPtr;

	if (!DeviceProxyManagerPtr.IsValid())
	{
		ITargetDeviceServicesModule* TargetDeviceServicesModule = FModuleManager::Get().LoadModulePtr<ITargetDeviceServicesModule>(TEXT("TargetDeviceServices"));
		if (TargetDeviceServicesModule)
		{
			DeviceProxyManagerPtr = TargetDeviceServicesModule->GetDeviceProxyManager();
		}
	}

	TSharedPtr<ITargetDeviceProxyManager> DeviceProxyManager = DeviceProxyManagerPtr.Pin();
	if (DeviceProxyManager.IsValid())
	{
		TSharedPtr<ITargetDeviceProxy> DeviceProxy = DeviceProxyManager->FindProxy(DeviceName);
		if (DeviceProxy.IsValid() && DeviceProxy->IsConnected() && DeviceProxy->IsAuthorized())
		{
			return true;
		}

		// check if this is an aggregate proxy
		TArray<TSharedPtr<ITargetDeviceProxy>> Devices;
		DeviceProxyManager->GetProxies(FName(*DeviceName), false, Devices);

		// returns true if the game can be launched al least on 1 device
		for (auto DevicesIt = Devices.CreateIterator(); DevicesIt; ++DevicesIt)
		{
			TSharedPtr<ITargetDeviceProxy> DeviceAggregateProxy = *DevicesIt;
			if (DeviceAggregateProxy.IsValid() && DeviceAggregateProxy->IsConnected() && DeviceAggregateProxy->IsAuthorized())
			{
				return true;
			}
		}

	}

	return false;
}

static void LaunchOnDevice(const FString& DeviceId, const FString& DeviceName, bool bUseTurnkey, FStringView Architecture)
{
	FTurnkeyEditorSupport::LaunchRunningMap(DeviceId, DeviceName, GetProjectPathForTurnkey(), bUseTurnkey, false, Architecture);
}

static void LaunchOnSimulator(const FString& DeviceId, const FString& DeviceName, bool bUseTurnkey, FStringView Architecture)
{								
	FTurnkeyEditorSupport::LaunchRunningMap(DeviceId, DeviceName, GetProjectPathForTurnkey(), false, true, Architecture);
}

static void PrepareLaunchOn(const FString& DeviceId, const FString& DeviceName)
{
	FTurnkeyEditorSupport::PrepareToLaunchRunningMap(DeviceId, DeviceName);
}

static void HandleLaunchOnDeviceActionExecute(const FString& DeviceId, const FString& DeviceName, bool bUseTurnkey, FStringView Architecture)
{
	PrepareLaunchOn(DeviceId, DeviceName);
	LaunchOnDevice(DeviceId, DeviceName, bUseTurnkey, Architecture);
}

static void HandleLaunchOnSimulatorActionExecute(const FString& DeviceId, const FString& DeviceName, bool bUseTurnkey, FStringView Architecture)
{
	LaunchOnSimulator(DeviceId, DeviceName, bUseTurnkey, Architecture);
}

static bool HandleLaunchOnDeviceActionCanExecute(FString DeviceName)
{
	return CanLaunchOnDevice(DeviceName);
}

static void GenerateDeviceProxyMenuParams(TSharedPtr<ITargetDeviceProxy> DeviceProxy, FName PlatformName, FUIAction& OutAction, FText& OutTooltip, FOnQuickLaunchSelected ExternalOnClickDelegate)
{
	// 	// create an All_<platform>_devices_on_<host> submenu
	// 	if (DeviceProxy->IsAggregated())
	// 	{
	// 		FString AggregateDevicedName(FString::Printf(TEXT("  %s"), *DeviceProxy->GetName())); //align with the other menu entries
	// 		FSlateIcon AggregateDeviceIcon(FAppStyle::Get().GetStyleSetName(), EditorPlatformInfo->GetIconStyleName(PlatformInfo::EPlatformIconSize::Normal));
	// 
	// 		MenuBuilder.AddSubMenu(
	// 			FText::FromString(AggregateDevicedName),
	// 			FText::FromString(AggregateDevicedName),
	// 			FNewMenuDelegate::CreateStatic(&MakeAllDevicesSubMenu, EditorPlatformInfo, DeviceProxy),
	// 			false, AggregateDeviceIcon, true
	// 		);
	// 		continue;
	// 	}

		// ... create an action...
	OutAction = FUIAction(
		FExecuteAction::CreateLambda([DeviceProxy, PlatformName, ExternalOnClickDelegate]()
			{
				if (!DeviceProxy->IsConnected())
				{
					DeviceProxy->PowerOn();
				}
			
				// only game and client devices are supported for launch on but devices only signal if they target client builds by their name e.g. "WindowsClient"
				// if the user has a EBuildTargetType::Client target selected we will launch on a client device, otherwise we fall back to the default game device

				// We need to use flavors to launch on correctly on Android_ASTC / Android_ETC2, etc
				const PlatformInfo::FTargetPlatformInfo* PlatformInfo = nullptr;
				if (FApp::IsInstalled())
				{
					PlatformInfo = PlatformInfo::FindPlatformInfo(PlatformName);
				}
				else
				{
					PlatformInfo = PlatformInfo::FindPlatformInfo(GetDefault<UPlatformsMenuSettings>()->GetTargetFlavorForPlatform(PlatformName));
				}

				FString VariantName = PlatformInfo->Name.ToString();

				// find out if the user selected a client build target 
				IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
				if (DesktopPlatform->GetTargetsForCurrentProject().Num() > 0)
				{
					TArray<FTargetInfo> Targets = DesktopPlatform->GetTargetsForCurrentProject();
					const FTargetInfo* TargetInfo = GetDefault<UPlatformsMenuSettings>()->GetLaunchOnTargetInfo();

					const TArray<FTargetInfo> ClientTargets = Targets.FilterByPredicate([](const FTargetInfo& Target)
						{
							return Target.Type == EBuildTargetType::Client;
						});

					// we just want to know if any client build is selected, the correct flavor was picked above
					for (const auto& Target : ClientTargets)
					{
						if (TargetInfo != nullptr && TargetInfo->Name == Target.Name)
						{
							VariantName += "Client";
							break;
						}
					}
				}

				// If the variant is not found it will fall back to default (NAME_None)
				FName DeviceVariantName = NAME_None;
				if (DeviceProxy->HasVariant(*VariantName))
				{
					DeviceVariantName = *VariantName;
				}

				FString DeviceId = DeviceProxy->GetTargetDeviceId(DeviceVariantName);
				if (DeviceProxy->IsSimulated())
				{
					PerPlatformLastSimChosen[*DeviceProxy->GetType()] = DeviceProxy->GetName();
					HandleLaunchOnSimulatorActionExecute(DeviceId, DeviceProxy->GetName(), true, DeviceProxy->GetArchitecture());
				}
				else
				{
					PerPlatformLastChosen[PlatformName] = DeviceProxy->GetName();
					HandleLaunchOnDeviceActionExecute(DeviceId, DeviceProxy->GetName(), true, DeviceProxy->GetArchitecture());
				}
				ExternalOnClickDelegate.ExecuteIfBound(DeviceId);
			}
	//		, FCanExecuteAction::CreateStatic(&HandleLaunchOnDeviceActionCanExecute, DeviceProxy->GetName())
	));

	// ... generate tooltip text
	FFormatNamedArguments TooltipArguments;
	TooltipArguments.Add(TEXT("DeviceID"), FText::FromString(DeviceProxy->GetName()));
	TooltipArguments.Add(TEXT("DisplayName"), FText::FromName(PlatformName));
	
	if (!DeviceProxy->IsAuthorized())
	{
		OutTooltip = FText::Format(LOCTEXT("LaunchDeviceToolTipText_UnauthorizedOrLocked", "{DisplayName} device ({DeviceID}) is unauthorized or locked"), TooltipArguments);
	}
	else if (!DeviceProxy->GetModel().IsEmpty() && !DeviceProxy->GetOSVersion().IsEmpty())
	{
		// Some platforms (ie: iOS/Android) can display additional information on mouse hover
		TooltipArguments.Add(TEXT("ModelId"), FText::FromString(DeviceProxy->GetModel()));
		TooltipArguments.Add(TEXT("OSVersion"), FText::FromString(DeviceProxy->GetOSVersion()));

		OutTooltip = FText::Format(LOCTEXT("LaunchDeviceToolTipText_ThisDeviceExtra", "Launch on \"{DeviceID}\" [OS:'{OSVersion}' Info:'{ModelId}']"), TooltipArguments);
	}
	else
	{
		OutTooltip = FText::Format(LOCTEXT("LaunchDeviceToolTipText_ThisDevice", "Launch the game on this {DisplayName} device ({DeviceID})"), TooltipArguments);
	}

	FProjectStatus ProjectStatus;
	if (IProjectManager::Get().QueryStatusForCurrentProject(ProjectStatus) && !ProjectStatus.IsTargetPlatformSupported(PlatformName))
	{
		FText TooltipLine2 = FText::Format(LOCTEXT("LaunchDevicePlatformWarning", "{DisplayName} is not listed as a target platform for this project, so may not run as expected."), TooltipArguments);
		OutTooltip = FText::Format(FText::FromString(TEXT("{0}\n\n{1}")), OutTooltip, TooltipLine2);
	}
}


void FTurnkeySupportModule::MakeQuickLaunchItems(class UToolMenu* Menu, FOnQuickLaunchSelected ExternalOnClickDelegate) const
{
	FToolMenuSection& MenuSection = Menu->AddSection("QuickLaunchDevices", LOCTEXT("Devices", "Devices"));

	MenuSection.AddDynamicEntry("PlatformsMenu", FNewToolMenuSectionDelegate::CreateLambda([ExternalOnClickDelegate](FToolMenuSection& DynamicSection)
	{
		TArray<FString> DeviceIdsToQuery;
		ITargetDeviceServicesModule* TargetDeviceServicesModule = static_cast<ITargetDeviceServicesModule*>(FModuleManager::Get().LoadModule(TEXT("TargetDeviceServices")));
		for (const auto& Pair : FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos())
		{
			if (Pair.Value.bIsFakePlatform)
			{
				continue;
			}

			FName PlatformName = Pair.Key;
			const FDataDrivenPlatformInfo& Info = Pair.Value;
			if (FDataDrivenPlatformInfoRegistry::IsPlatformHiddenFromUI(PlatformName))
			{
				continue;
			}

			// look for devices for all platforms, even if the platform isn't installed - Turnkey can install Sdk after selecting LaunchOn
			TArray<TSharedPtr<ITargetDeviceProxy>> DeviceProxies;
			TargetDeviceServicesModule->GetDeviceProxyManager()->GetAllProxies(PlatformName, DeviceProxies);
			
			const ITargetPlatform* Platform = GetTargetPlatformManager()->FindTargetPlatform(PlatformName.ToString());

			// Remove Aggregate Device
			if (Platform && !Platform->SupportsFeature(ETargetPlatformFeatures::ShowAggregateDevice))
			{
				DeviceProxies = DeviceProxies.FilterByPredicate([](const TSharedPtr<ITargetDeviceProxy>& Device)
				{
					return !Device->IsAggregated();
				});
			}

			// Remove any Simulator based devices
			DeviceProxies = DeviceProxies.FilterByPredicate([](const TSharedPtr<ITargetDeviceProxy>& Device)
			{
				return Device->GetConnectionType() != TEXT("Simulator");
			});
			
			if (DeviceProxies.Num() > 0)
			{
				FString LastChosen;
				if (PerPlatformLastChosen.Contains(PlatformName))
				{
					LastChosen = PerPlatformLastChosen[PlatformName];
				}
				else
				{
					LastChosen = DeviceProxies[0]->GetName();
					PerPlatformLastChosen.Add(PlatformName, LastChosen);
				}
				Algo::Sort(DeviceProxies, [&LastChosen = std::as_const(LastChosen)](TSharedPtr<ITargetDeviceProxy> A, TSharedPtr<ITargetDeviceProxy> B)
						   { return A->GetName() == LastChosen; });


				if (DeviceProxies.Num() == 1)
				{
					FUIAction Action;
					FText Tooltip;
					GenerateDeviceProxyMenuParams(DeviceProxies[0], PlatformName, Action, Tooltip, ExternalOnClickDelegate);

					DynamicSection.AddMenuEntry(
						NAME_None,
						MakeSdkStatusAttribute(PlatformName, DeviceProxies[0]),
						Tooltip,
						MakePlatformSdkIconAttribute(PlatformName, DeviceProxies[0]),
						Action
					);
				}
				else
				{
					DynamicSection.AddSubMenu(
						NAME_None,
						MakeSdkStatusAttribute(PlatformName, nullptr),
						FText::FromString(PlatformName.ToString()),
						FNewToolMenuDelegate::CreateLambda(
							[TargetDeviceServicesModule, PlatformName, LastChosen, ExternalOnClickDelegate](UToolMenu* SubToolMenu)
							{
								FToolMenuSection& Section = SubToolMenu->AddSection(NAME_None);

								// re-get the proxies, just in case they changed
								TArray<TSharedPtr<ITargetDeviceProxy>> DeviceProxies;
								TargetDeviceServicesModule->GetDeviceProxyManager()->GetAllProxies(PlatformName, DeviceProxies);
								const ITargetPlatform* Platform = GetTargetPlatformManager()->FindTargetPlatform(PlatformName.ToString());

								// Remove Aggregate Device
								if (Platform && !Platform->SupportsFeature(ETargetPlatformFeatures::ShowAggregateDevice))
								{
									DeviceProxies = DeviceProxies.FilterByPredicate([](const TSharedPtr<ITargetDeviceProxy>& Device)
									{
										return !Device->IsAggregated();
									});
								}

								// Remove any Simulator based devices
								DeviceProxies = DeviceProxies.FilterByPredicate([](const TSharedPtr<ITargetDeviceProxy>& Device)
								{
									return Device->GetConnectionType() != TEXT("Simulator");
								});
								
								Algo::Sort(DeviceProxies, [&LastChosen = std::as_const(LastChosen)](TSharedPtr<ITargetDeviceProxy> A, TSharedPtr<ITargetDeviceProxy> B)
									{ return A->GetName() == LastChosen; });

								// for each one, put an entry (even the one that was in the outer menu, for less confusion)
								for (const TSharedPtr<ITargetDeviceProxy>& Proxy : DeviceProxies)
								{
									FUIAction SubAction;
									FText SubTooltip;
									GenerateDeviceProxyMenuParams(Proxy, PlatformName, SubAction, SubTooltip, ExternalOnClickDelegate);
									
									Section.AddMenuEntry(
										NAME_None,
										MakeSdkStatusAttribute(PlatformName, Proxy),
										SubTooltip,
										MakePlatformSdkIconAttribute(PlatformName, Proxy),
										SubAction,
										EUserInterfaceActionType::Button
									);
								}
							}),
						{},
						EUserInterfaceActionType::Check,
						false,
						MakePlatformSdkIconAttribute(PlatformName, nullptr),
						true
					);
				}

				ITurnkeySupportModule& TurnkeySupport = ITurnkeySupportModule::Get();
				// gather any unknown status devices to query at the end
				for (const TSharedPtr<ITargetDeviceProxy>& Proxy : DeviceProxies)
				{
					FString DeviceId = Proxy->GetTargetDeviceId(Proxy->GetVanillaPlatformId(NAME_None));
					if (TurnkeySupport.GetSdkInfoForDeviceId(DeviceId).Status == ETurnkeyPlatformSdkStatus::Unknown)
					{
						DeviceIdsToQuery.Add(DeviceId);
					}
				}
			}
		}
		// now kick-off any devices that need to be updated
		if (DeviceIdsToQuery.Num() > 0)
		{
			ITurnkeySupportModule::Get().UpdateSdkInfoForDevices(DeviceIdsToQuery);
		}
	}
	));


	UProjectPackagingSettings* AllPlatformPackagingSettings = GetMutableDefault<UProjectPackagingSettings>();

	FToolMenuSection& ConfigSection = Menu->AddSection("BuildConfig", LOCTEXT("TurnkeySection_BuildConfig", "Build Configuration"));
	{
		const UProjectPackagingSettings::FConfigurationInfo& DefaultConfigInfo = UProjectPackagingSettings::ConfigurationInfo[static_cast<int32>(AllPlatformPackagingSettings->BuildConfiguration)];
		ConfigSection.AddMenuEntry(
			NAME_None,
			FText::Format(LOCTEXT("DefaultConfiguration",  "Use Project Setting ({0})"), DefaultConfigInfo.Name),
			DefaultConfigInfo.ToolTip,
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::SetLaunchOnBuildConfiguration, EProjectPackagingBuildConfigurations::PPBC_MAX),
				FCanExecuteAction(),
				FIsActionChecked::CreateStatic(&FTurnkeySupportCallbacks::LaunchOnBuildConfigurationIsChecked, EProjectPackagingBuildConfigurations::PPBC_MAX)
			),
			EUserInterfaceActionType::RadioButton
		);

		EProjectType ProjectType = FTurnkeyEditorSupport::DoesProjectHaveCode() ? EProjectType::Code : EProjectType::Content;
		TArray<EProjectPackagingBuildConfigurations> PackagingConfigurations = UProjectPackagingSettings::GetValidPackageConfigurations();

		for (EProjectPackagingBuildConfigurations PackagingConfiguration : PackagingConfigurations)
		{
			const UProjectPackagingSettings::FConfigurationInfo& ConfigurationInfo = UProjectPackagingSettings::ConfigurationInfo[(int)PackagingConfiguration];
			if (FInstalledPlatformInfo::Get().IsValid(TOptional<EBuildTargetType>(), TOptional<FString>(), ConfigurationInfo.Configuration, ProjectType, EInstalledPlatformState::Downloaded))
			{
				ConfigSection.AddMenuEntry(
					NAME_None,
					ConfigurationInfo.Name,
					ConfigurationInfo.ToolTip,
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::SetLaunchOnBuildConfiguration, PackagingConfiguration),
						FCanExecuteAction(),
						FIsActionChecked::CreateStatic(&FTurnkeySupportCallbacks::LaunchOnBuildConfigurationIsChecked, PackagingConfiguration)
					),
					EUserInterfaceActionType::RadioButton
				);
			}
		}
	}

	FProjectStatus ProjectStatus;
	bool bHasCode = IProjectManager::Get().QueryStatusForCurrentProject(ProjectStatus) && ProjectStatus.bCodeBasedProject;

	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	TArray<FTargetInfo> Targets = bHasCode ? DesktopPlatform->GetTargetsForCurrentProject() : DesktopPlatform->GetTargetsForProject(FString());
	const TArray<FTargetInfo> ValidTargets = Targets.FilterByPredicate([](const FTargetInfo& Target)
		{
			return Target.Type == EBuildTargetType::Game;// || Target.Type == EBuildTargetType::Client || Target.Type == EBuildTargetType::Server;
		});

	if (ValidTargets.Num() > 1)
	{
		FToolMenuSection& LaunchTargetSection = Menu->AddSection("QuickLaunchTarget", LOCTEXT("QuickLaunchTarget", "Quick Launch Game Target"));

		for (const FTargetInfo& Target : ValidTargets)
		{
			LaunchTargetSection.AddMenuEntry(
				NAME_None,
				FText::FromString(Target.Name),
				FText::Format(LOCTEXT("PackageTargetName", "Package the '{0}' target."), FText::FromString(Target.Name)),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::SetLaunchOnBuildTarget, Target.Name),
					FCanExecuteAction(),
					FIsActionChecked::CreateStatic(&FTurnkeySupportCallbacks::LaunchOnBuildTargetIsChecked, Target.Name)
				),
				EUserInterfaceActionType::RadioButton
			);
		}

	}

	// if we don't have an external delegate to call, then this is the internally included items in the Platforms menu and we can add the extra option(s)
	if (!ExternalOnClickDelegate.IsBound())
	{
		FToolMenuSection& CookOnTheFlySection = Menu->AddSection("CookOnTheFlySection");

		CookOnTheFlySection.AddSeparator(NAME_None);

		CookOnTheFlySection.AddMenuEntry(
			NAME_None,
			LOCTEXT("CookOnTheFlyOnLaunch", "Cook on the Fly"),
			LOCTEXT("CookOnTheFlyOnLaunchDescription", "Cook on the fly instead of cooking upfront when launching"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "MainFrame.CookContent"),
			FUIAction(
				FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::SetCookOnTheFly),
				FCanExecuteAction(),
				FIsActionChecked::CreateStatic(&FTurnkeySupportCallbacks::SetCookOnTheFlyIsChecked)
			),
			EUserInterfaceActionType::ToggleButton
		);
	}
}

TSharedRef<SWidget> FTurnkeySupportModule::MakeTurnkeyMenuWidget() const
{
	FTurnkeySupportCommands::Register();
	const FTurnkeySupportCommands& Commands = FTurnkeySupportCommands::Get();

	static const FName MenuName("LevelEditor.MainMenu.Platforms");

	if (!UToolMenus::Get()->IsMenuRegistered(MenuName))
	{
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(MenuName);

		FToolMenuSection& LaunchSection = Menu->AddSection("Launch", LOCTEXT("TurnkeySection_Launch", "Launch"));
		{
			LaunchSection.AddSubMenu(
				"QuickLaunchSubMenu",
				LOCTEXT("QuickLaunch", "Quick Launch Current Level"),
				LOCTEXT("QuickLaunch_ToolTip", "Launch only the current level on the device for fast iteration"),
				FNewToolMenuDelegate::CreateLambda([this](UToolMenu* SubMenu)
					{
						FOnQuickLaunchSelected EmptyFunc;
						MakeQuickLaunchItems(SubMenu, EmptyFunc);
					}
				),
				false, /* bInOpenSubMenuOnClick */
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Launcher.TabIcon")
			);
		}

		FToolMenuSection& ProjectLauncherSection = Menu->AddSection("ProjectLauncher", LOCTEXT("TurnkeySection_ProjectLauncher", "Project Launcher"));
		{
			if (IPluginManager::Get().FindEnabledPlugin(TEXT("ProjectLauncher")) != nullptr)
			{
				ProjectLauncherSection.AddMenuEntry(
					NAME_None,
					LOCTEXT("OpenProjectLauncher", "Project Launcher..."),
					LOCTEXT("OpenProjectLauncher_ToolTip", "Open the Project Launcher for advanced packaging, deploying and launching of your projects"),
					FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Launcher.TabIcon"),
					FUIAction(FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::OpenProjectLauncher))
				);
			}
		}

		FToolMenuSection& ContentManagementSection = Menu->AddSection("ContentManagementSection", LOCTEXT("TurnkeySection_ContentManagement", "Cook or Package"));
		{
			ContentManagementSection.AddSubMenu(
				"CookSubMenu",
				LOCTEXT("Turnkey_CookContent", "Cook Content"),
				LOCTEXT("TurnkeyTooltip_CookContentSubMenu", "Cook this project"),
				FNewToolMenuDelegate::CreateLambda([this](UToolMenu* SubMenu)
					{
						MakeCookContentMenuItems(SubMenu);
					}
				),
				false,
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "MainFrame.CookContent")
			);

			ContentManagementSection.AddSubMenu(
				"PackageSubMenu",
				LOCTEXT("Turnkey_PackageProject", "Package Project"),
				LOCTEXT("TurnkeyTooltip_PackageProject", "Package this project and archive it to a user-selected directory. This can then be used to install and run."),
				FNewToolMenuDelegate::CreateLambda([this](UToolMenu* SubMenu)
					{
						MakePackageProjectMenuItems(SubMenu);
					}
				),
				false,
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "MainFrame.PackageProject")
			);

#if PLATFORM_WINDOWS || PLATFORM_MAC
			// Prepare For Debugging is only on iOS and tvOS platforms. The menu entry is not visible unless those
			// platforms are supported.
			{
				TArray<FName> SupportedPlatforms;
				GetTurnkeyCategorizedPlatforms(&SupportedPlatforms, nullptr, nullptr);

				if (SupportedPlatforms.Contains(TEXT("IOS")) || SupportedPlatforms.Contains(TEXT("TvOS")))
				{
					ContentManagementSection.AddSubMenu(
						"PrepareSubMenu",
						LOCTEXT("Turnkey_PrepareForDebugging", "Prepare For Debugging"),
						LOCTEXT("TurnkeyTooltip_PrepareForDebugging", "Create files to help streamline the debugging pipeline on some platforms"),
						FNewToolMenuDelegate::CreateLambda([this](UToolMenu* SubMenu)
							{
								MakePrepareForDebuggingMenuItems(SubMenu);
							}
						),
						false,
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "Debug")
					);
				}
			}
#endif
		}

		// Custom builds menu is only displayed when custom builds have been configured.
		FToolMenuSection& CustomBuildSection = Menu->AddSection("CustomBuildsSection", LOCTEXT("TurnkeySection_CustomBuilds", "Custom Builds"));
		{
			TArray<FName> SupportedPlatforms;
			GetTurnkeyCategorizedPlatforms(&SupportedPlatforms, nullptr, nullptr);

			bool bHasCustomBuilds = false;

			for (const FName& IniPlatformName : SupportedPlatforms)
			{
				UProjectPackagingSettings* PackagingSettings = FTurnkeySupportCallbacks::GetPackagingSettingsForPlatform(IniPlatformName);

				if (!PackagingSettings->EngineCustomBuilds.IsEmpty() || !PackagingSettings->ProjectCustomBuilds.IsEmpty())
				{
					bHasCustomBuilds = true;
					break;
				}
			}

			if (bHasCustomBuilds)
			{
				CustomBuildSection.AddSubMenu(
					"CustomBuildsSubMenu",
					LOCTEXT("Turnkey_CustomBuilds", "Custom Builds"),
					LOCTEXT("TurnkeyTooltip_CustomBuilds", "Run custom builds commands defined in config files"),
					FNewToolMenuDelegate::CreateLambda([this](UToolMenu* SubMenu)
						{
							MakeCustomBuildsMenuItems(SubMenu);
						}
					),
					false,
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Advanced")
				);
			}
		}

		FToolMenuSection& OptionsSection = Menu->AddSection("Options", LOCTEXT("TurnkeySection_Options", "Options and Settings"));
		{
			OptionsSection.AddSubMenu("SDKManagementSubMenu",
				LOCTEXT("SDKManagement", "SDK Management"),
				LOCTEXT("SDKManagement_ToolTip", "Manage SDK Installations"),
				FNewToolMenuDelegate::CreateLambda([this](UToolMenu* SubMenu)
					{
						MakeSdkManagementMenuItems(SubMenu);	
					}
				),
				false,
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "MainFrame.Project")
			);

			OptionsSection.AddMenuEntry(
				NAME_None,
				LOCTEXT("OpenDeviceManager", "Device Manager..."),
				LOCTEXT("OpenDeviceManager_ToolTip", "View and manage connected devices."),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "DeviceDetails.TabIcon"),
				FUIAction(FExecuteAction::CreateStatic(&FTurnkeySupportCallbacks::OpenDeviceManager))
			);

			FTurnkeyEditorSupport::AddEditorOptions(OptionsSection);
		}
	}

#if WITH_EDITOR
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	FToolMenuContext MenuContext(FTurnkeySupportCommands::ActionList, LevelEditorModule.GetToolBarExtensibilityManager()->GetAllExtenders());
#else
	FToolMenuContext MenuContext(FTurnkeySupportCommands::ActionList);
#endif
	return UToolMenus::Get()->GenerateWidget(MenuName, MenuContext);

}

void FTurnkeySupportModule::MakeTurnkeyMenu(FToolMenuSection& MenuSection, const FToolMenuInsert& InsertPosition) const
{
	// make sure the DeviceProxyManager is going _before_ we create the menu contents dynamically, so that devices will show up
	ITargetDeviceServicesModule* TargetDeviceServicesModule = static_cast<ITargetDeviceServicesModule*>(FModuleManager::Get().LoadModule(TEXT("TargetDeviceServices")));
	TargetDeviceServicesModule->GetDeviceProxyManager();

	FToolMenuEntry& PlatformsEntry = MenuSection.AddSubMenu(
		"Platforms",
		LOCTEXT("PlatformMenu", "Platforms"),
		LOCTEXT("PlatformMenu_Tooltip", "Platform related actions and settings (Launching, Packaging, custom builds, etc)"),
		FOnGetContent::CreateLambda([this]{ return MakeTurnkeyMenuWidget(); })
	);

	PlatformsEntry.InsertPosition = InsertPosition;
}

// some shared functionality
static void PrepForTurnkeyReport(FString& BaseCommandline, FString& ReportFilename)
{
	// make sure intermediate directory exists
	IFileManager::Get().MakeDirectory(*FPaths::ProjectIntermediateDir());

	const FString ProjectPath = GetProjectPathForTurnkey();
	// now pass a project to UAT
	if (!ProjectPath.IsEmpty())
	{
		BaseCommandline.Appendf(TEXT(" -ScriptsForProject=\"%s\" "), *ProjectPath);
	}

	FString LogFilename;
	FString LogAndReportParams = FTurnkeySupportCallbacks::GetLogAndReportCommandline(LogFilename, ReportFilename);

	BaseCommandline = BaseCommandline.Appendf(TEXT("Turnkey -utf8output -WaitForUATMutex -command=VerifySdk %s"), *LogAndReportParams);
	// now pass a project to Turnkey
	if (!ProjectPath.IsEmpty())
	{
		BaseCommandline.Appendf(TEXT(" -project=\"%s\" "), *ProjectPath);
	}
}

bool GetSdkInfoFromTurnkey(FString Line, FName& PlatformName, FString& DeviceId, FTurnkeySdkInfo& SdkInfo)
{
	int32 Colon = Line.Find(TEXT(": "));

	if (Colon < 0)
	{
		return false;
	}

	// break up the string
	FString PlatformString = Line.Mid(0, Colon);
	FString Info = Line.Mid(Colon + 2);

	int32 AtSign = PlatformString.Find(TEXT("@"));
	if (AtSign > 0)
	{
		// return the platform@name as the deviceId, then remove the @name part for the platform
		DeviceId = ConvertToDDPIDeviceId(PlatformString);
		PlatformString = PlatformString.Mid(0, AtSign);
	}

	// get the DDPI name
	PlatformName = FName(*ConvertToDDPIPlatform(PlatformString));

	// parse out the results from the (key=val, key=val) result from turnkey
	FString StatusString;
	FString FlagsString;
	FParse::Value(*Info, TEXT("Status="), StatusString);
	FParse::Value(*Info, TEXT("Flags="), FlagsString);
	FString ErrorString;
	FParse::Value(*Info, TEXT("Error="), ErrorString);
	SdkInfo.SdkErrorInformation = FText::FromString(ErrorString.Replace(TEXT("|"), TEXT("\n")));

	FString SDKNamesString;
	TArray<FString> SDKNames;
	if (FParse::Value(*Info, TEXT("SDKs="), SDKNamesString))
	{
		SDKNamesString.ParseIntoArray(SDKNames, TEXT(","), true);
	}
	else
	{
		SDKNames.Add("SDK");
	}
	if (DeviceId.Len() == 0)
	{
		SDKNames.Add("AutoSDK");
	}

	for (const FString& Name : SDKNames)
	{
		FString Min, Max, Current;
		// handle both AutoSDK and manual (auto only has Allowed_AutoSDK, others have Min/Max)
		FParse::Value(*Info, *FString::Printf(TEXT("MinAllowed_%s="), *Name), Min);
		FParse::Value(*Info, *FString::Printf(TEXT("MaxAllowed_%s="), *Name), Max);
		FParse::Value(*Info, *FString::Printf(TEXT("Allowed_%s="), *Name), Min);
		FParse::Value(*Info, *FString::Printf(TEXT("Allowed_%s="), *Name), Max);
		FParse::Value(*Info, *FString::Printf(TEXT("Current_%s="), *Name), Current);
		// also handle no name at all (for device, etc)
		FParse::Value(*Info, TEXT("MinAllowed="), Min);
		FParse::Value(*Info, TEXT("MaxAllowed="), Max);
		FParse::Value(*Info, TEXT("Allowed="), Min);
		FParse::Value(*Info, TEXT("Allowed="), Max);
		FParse::Value(*Info, TEXT("Current="), Current);
		SdkInfo.SDKVersions.Add(Name, { Min, Max, Current });
	}

	SdkInfo.Status = ETurnkeyPlatformSdkStatus::Unknown;
	if (StatusString == TEXT("Valid"))
	{
		SdkInfo.Status = ETurnkeyPlatformSdkStatus::Valid;
	}
	else
	{
		if (FlagsString.Contains(TEXT("AutoSdk_InvalidVersionExists")) || FlagsString.Contains(TEXT("InstalledSdk_InvalidVersionExists")))
		{
			SdkInfo.Status = ETurnkeyPlatformSdkStatus::OutOfDate;
		}
		else
		{
			SdkInfo.Status = ETurnkeyPlatformSdkStatus::NoSdk;
		}
	}
	SdkInfo.bCanInstallFullSdk = FlagsString.Contains(TEXT("Support_FullSdk"));
	SdkInfo.bCanInstallAutoSdk = FlagsString.Contains(TEXT("Support_AutoSdk"));
	SdkInfo.bHasBestSdk = FlagsString.Contains(TEXT("Sdk_HasBestVersion"));

	SdkInfo.DeviceStatus = ETurnkeyDeviceStatus::Unknown;
	if (FlagsString.Contains(TEXT("Device_InvalidPrerequisites")))
	{
		SdkInfo.DeviceStatus = ETurnkeyDeviceStatus::InvalidPrerequisites;
	}
	else if (FlagsString.Contains(TEXT("Device_InstallSoftwareValid")))
	{
		SdkInfo.DeviceStatus = ETurnkeyDeviceStatus::SoftwareValid;
	}
	else if (FlagsString.Contains(TEXT("Device_InstallSoftwareInvalid")))
	{
		SdkInfo.DeviceStatus = ETurnkeyDeviceStatus::SoftwareInvalid;
	}

	// Device auto update support
	SdkInfo.DeviceAutoSoftwareUpdates = ETurnkeyDeviceAutoSoftwareUpdateMode::Unknown;
	if (FlagsString.Contains(TEXT("Device_AutoSoftwareUpdates_Disabled")))
	{
		SdkInfo.DeviceAutoSoftwareUpdates = ETurnkeyDeviceAutoSoftwareUpdateMode::Disabled;
	}
	else if (FlagsString.Contains(TEXT("Device_AutoSoftwareUpdates_Enabled")))
	{
		SdkInfo.DeviceAutoSoftwareUpdates = ETurnkeyDeviceAutoSoftwareUpdateMode::Enabled;
	}

	return true;
}


static constexpr bool bDeleteTurnkeyProcessOnCompletion = (PLATFORM_WINDOWS);


void FTurnkeySupportModule::UpdateSdkInfo()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTurnkeySupportModule::UpdateSdkInfo);

	// make sure all known platforms are in the map
	if (PerPlatformSdkInfo.Num() == 0)
	{
		for (auto& It : FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos())
		{
			PerPlatformSdkInfo.Add(It.Key, FTurnkeySdkInfo());
		}
	}

	// don't run UAT from commandlets (like the cooker) that are often launched from UAT and this will go poorly
	if (IsRunningCommandlet() || FApp::IsUnattended())
	{
		return;
	}


	FString BaseCommandline, ReportFilename;
	PrepForTurnkeyReport(BaseCommandline, ReportFilename);
	// get status for all platforms
	FString Commandline = BaseCommandline + TEXT(" -platform=all");

	UE_LOGF(LogTurnkeySupport, Log, "Running Turnkey SDK detection: '%ls'", *Commandline);

	{
		FScopeLock Lock(&GTurnkeySection);

		// reset status to unknown
		for (auto& It : PerPlatformSdkInfo)
		{
			It.Value.Status = ETurnkeyPlatformSdkStatus::Querying;
		}

		// reset the per-device status when querying general Sdk status
		ClearDeviceStatus();
	}

	FSerializedUATProcess* TurnkeyProcess = new FSerializedUATProcess(Commandline);

	// block the game thread on exit while TurnkeyProcess being cancelled
	struct FEventDeleter { void operator()(FEvent* Event) { FPlatformProcess::ReturnSynchEventToPool(Event); } };
	TSharedPtr<FEvent, ESPMode::ThreadSafe> Barrier{ FPlatformProcess::GetSynchEventFromPool(), FEventDeleter{} };

	TurnkeyProcess->OnCanceled().BindLambda([Barrier] { Barrier->Trigger(); });
	FDelegateHandle OnExitHandle = FCoreDelegates::OnExit.AddLambda([TurnkeyProcess, Barrier] { TurnkeyProcess->Cancel(); Barrier->Wait(); });

	TurnkeyProcess->OnCompleted().BindLambda([this, ReportFilename, TurnkeyProcess, Barrier, OnExitHandle](int32 ExitCode)
	{
		UE_LOGF(LogTurnkeySupport, Log, "Completed SDK detection: ExitCode = %d", ExitCode);

		AsyncTask(ENamedThreads::GameThread, [this, ReportFilename, ExitCode, TurnkeyProcess, OnExitHandle]()
		{
			if (IsEngineExitRequested())
			{
				return;
			}

			FScopeLock Lock(&GTurnkeySection);

			if (ExitCode == 0 || ExitCode == 10)
			{
				TArray<FString> Contents;
				if (FFileHelper::LoadFileToStringArray(Contents, *ReportFilename))
				{
					for (FString& Line : Contents)
					{
 						UE_LOGF(LogTurnkeySupport, Log, "Turnkey Platform: %ls", *Line);

						// parse a Turnkey line
						FName PlatformName;
						FString Unused;
						FTurnkeySdkInfo SdkInfo;
						if (GetSdkInfoFromTurnkey(Line, PlatformName, Unused, SdkInfo) == false)
						{
							continue;
						}

						// we received a platform from UAT that we don't know about in the editor. this can happen if you have a UBT/UAT that was compiled with platform access
						// but then you are running without that platform synced. skip this platform and move on
						if (!PerPlatformSdkInfo.Contains(PlatformName))
						{
							UE_LOGF(LogTurnkeySupport, Log, "Received platform %ls from Turnkey, but the engine doesn't know about it. Skipping...", *PlatformName.ToString());
							continue;
						}

						// check if we had already set a ManualSDK - and don't set it again. Because of the way AutoSDKs are activated in the editor after the first call to Turnkey,
						// future calls to Turnkey will inherit the AutoSDK env vars, and it won't be able to determine the manual SDK versions anymore. If we use the editor to
						// install an SDK via Turnkey, it will directly update the installed version based on the result of that command, not this Update operation

						TMap<FString, FTurnkeySdkInfo::Version> OriginalVersions = PerPlatformSdkInfo[PlatformName].SDKVersions;

						// set it into the platform
						PerPlatformSdkInfo[PlatformName] = SdkInfo;

						// restore the original installed version if it set after the first time, except for AutoVersion
						if (OriginalVersions.Num() > 0)
						{
							PerPlatformSdkInfo[PlatformName].SDKVersions = OriginalVersions;
							if (SdkInfo.SDKVersions.Contains(TEXT("AutoSDK")))
							{
								PerPlatformSdkInfo[PlatformName].SDKVersions.Add(TEXT("AutoSDK"), SdkInfo.SDKVersions[TEXT("AutoSDK")]);
							}
						}

						// initialize default flavor if not yet initialized
						// flavor selection exists for Android to be able to package ASTC, DXT, ETC2, etc
						const PlatformInfo::FTargetPlatformInfo* VanillaInfo = PlatformInfo::FindVanillaPlatformInfo(PlatformName);
						if (VanillaInfo != nullptr)
						{
							// gather all valid flavors
							const TArray<const PlatformInfo::FTargetPlatformInfo*> ValidFlavors = VanillaInfo->Flavors.FilterByPredicate([](const PlatformInfo::FTargetPlatformInfo* Target)
								{
									// Editor isn't a valid platform type that users can target
									// The Build Target will choose client or server, so no need to show them as well
									return Target->PlatformType != EBuildTargetType::Editor && Target->PlatformType != EBuildTargetType::Client && Target->PlatformType != EBuildTargetType::Server;
								});

							// if platform uses flavors and doesn't already have a default, set default flavor to the first availabl
							if (ValidFlavors.Num() > 1)
							{
								FName CurrentFlavor = GetDefault<UPlatformsMenuSettings>()->GetTargetFlavorForPlatform(PlatformName);

								if (CurrentFlavor == VanillaInfo->Name)
								{
									UE_LOGF(LogTurnkeySupport, Log, "Platform %ls uses flavors but has no default flavor set. Setting to %ls", *PlatformName.ToString(), *(ValidFlavors[0]->Name.ToString()));
									FTurnkeySupportCallbacks::SetActiveFlavor(ValidFlavors[0]);
								}
							}
						}
					}
				}

				// update all deviecs
				UpdateSdkInfoForAllDevices();
			}
			else
			{
				for (auto& It : PerPlatformSdkInfo)
				{
					It.Value.Status = ETurnkeyPlatformSdkStatus::Error;
					It.Value.SdkErrorInformation = FText::Format(NSLOCTEXT("Turnkey", "TurnkeyError_ReturnedError", "Turnkey returned an error, code {0} (See log)"), { ExitCode });
				}
				UE_LOGF(LogTurnkeySupport, Warning, "Turnkey failed to run properly, full Turnkey output:\n%ls", *TurnkeyProcess->GetFullOutputWithoutDelegate());
			}


			for (auto& It : PerPlatformSdkInfo)
			{
				if (It.Value.Status == ETurnkeyPlatformSdkStatus::Querying)
				{
					// fake platforms won't come back, just skip it
					if (FDataDrivenPlatformInfoRegistry::GetPlatformInfo(It.Key).bIsFakePlatform)
					{
						It.Value.Status = ETurnkeyPlatformSdkStatus::Unknown;
					}
					else
					{
						It.Value.Status = ETurnkeyPlatformSdkStatus::Error;
						It.Value.SdkErrorInformation = NSLOCTEXT("Turnkey", "TurnkeyError_NotReturned", "The platform's Sdk status was not returned from Turnkey");
					}
				}
			}

			// cleanup
			FCoreDelegates::OnExit.Remove(OnExitHandle);
			delete TurnkeyProcess;
			IFileManager::Get().Delete(*ReportFilename);
		});

		Barrier->Trigger();
	});

	TurnkeyProcess->Launch();
}

void FTurnkeySupportModule::UpdateSdkInfoForAllDevices()
{
	TArray<FString> HostDevices;
	TArray<FString> OtherDevices;

	{
		FScopeLock Lock(&GTurnkeySection);

		// now kick off status update for all devices (host platform first, then everything else)
		ITargetDeviceServicesModule* TargetDeviceServicesModule = static_cast<ITargetDeviceServicesModule*>(FModuleManager::Get().LoadModule(TEXT("TargetDeviceServices")));
		for (const auto& Pair : FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos())
		{
			FName PlatformName = Pair.Key;
			if (!Pair.Value.bIsFakePlatform && !FDataDrivenPlatformInfoRegistry::IsPlatformHiddenFromUI(PlatformName))
			{
				ITargetPlatform* TP = GetTargetPlatformManager()->FindTargetPlatform(PlatformName);
				if (TP == nullptr)
				{
					continue;
				}

				TArray<ITargetDevicePtr> Devices;
				TP->GetAllDevices(Devices);

				for (ITargetDevicePtr Device : Devices)
				{
					if (!Device.IsValid())
					{
						UE_LOGF(LogTurnkeySupport, Log, "Platform %ls returned an invlid device from GetAllDevices, which is not expected", *PlatformName.ToString());
						continue;
					}
					FString DeviceId = Device->GetId().ToString();
					if (GetSdkInfoForDeviceId(DeviceId).Status == ETurnkeyPlatformSdkStatus::Unknown)
					{
						if (Pair.Value.IniPlatformName == FPlatformProperties::IniPlatformName())
						{
							HostDevices.Add(DeviceId);
						}
						else
						{
							OtherDevices.Add(DeviceId);
						}
					}
				}
			}
		}
	}

	UpdateSdkInfoForDevices(HostDevices);
	UpdateSdkInfoForDevices(OtherDevices);
}

void FTurnkeySupportModule::UpdateSdkInfoForProxy(const TSharedRef<ITargetDeviceProxy>& AddedProxy)
{
	bool bIsNeeded;

	FString DeviceId = AddedProxy->GetTargetDeviceId(AddedProxy->GetVanillaPlatformId(NAME_None));
	{
		FScopeLock Lock(&GTurnkeySection);
		bIsNeeded = GetSdkInfoForDeviceId(DeviceId).Status == ETurnkeyPlatformSdkStatus::Unknown;
	}

	if (bIsNeeded)
	{
		UpdateSdkInfoForDevices({ DeviceId });
	}
}

void FTurnkeySupportModule::UpdateSdkInfoForDevices(TArray<FString> PlatformDeviceIds)
{
#if PLATFORM_MAC_ARM64
	// Dummy local mac device UUID, no need to update
	PlatformDeviceIds.Remove(TEXT("IOS@") + LocalMacDummyUUID);
#endif
	
	if (PlatformDeviceIds.Num() == 0)
	{
		return;
	}

	FString BaseCommandline, ReportFilename;
	PrepForTurnkeyReport(BaseCommandline, ReportFilename);

	// the platform part of the Id may need to be converted to be turnkey (ie UBT) proper

	FString Commandline = BaseCommandline + FString(TEXT(" -Device=")) + FString::JoinBy(PlatformDeviceIds, TEXT("+"), [](FString Id) { return ConvertToUATDeviceId(Id); });

	UE_LOGF(LogTurnkeySupport, Log, "Running Turnkey device detection: '%ls'", *Commandline);

	{
		FScopeLock Lock(&GTurnkeySection);

		// set status to querying
		FTurnkeySdkInfo DefaultInfo;
		DefaultInfo.Status = ETurnkeyPlatformSdkStatus::Querying;
		for (const FString& Id : PlatformDeviceIds)
		{
			PerDeviceSdkInfo.Add(ConvertToDDPIDeviceId(Id), DefaultInfo);
		}
	}

	FSerializedUATProcess* TurnkeyProcess = new FSerializedUATProcess(Commandline);
	TurnkeyProcess->OnCompleted().BindLambda([this, ReportFilename, PlatformDeviceIds, TurnkeyProcess](int32 ExitCode)
	{
		UE_LOGF(LogTurnkeySupport, Log, "Completed device detection: Code = %d", ExitCode);

		AsyncTask(ENamedThreads::GameThread, [this, ReportFilename, PlatformDeviceIds, ExitCode, TurnkeyProcess]()
		{
			LLM_SCOPE(ELLMTag::EngineMisc);
			FScopeLock Lock(&GTurnkeySection);

			if (!IsEngineExitRequested() && (ExitCode == 0 || ExitCode == 10))
			{
				TArray<FString> Contents;
				if (FFileHelper::LoadFileToStringArray(Contents, *ReportFilename))
				{
					for (FString& Line : Contents)
					{
						FName PlatformName;
						FString DDPIDeviceId;
						FTurnkeySdkInfo SdkInfo;
						if (GetSdkInfoFromTurnkey(Line, PlatformName, DDPIDeviceId, SdkInfo) == false)
						{
							continue;
						}

						// skip over non-device lines
						if (DDPIDeviceId.Len() == 0)
						{
							continue;
						}

						// we received a device from UAT that we don't know about in the editor. this should never happen since we pass a list of devices to Turnkey, 
						// so this is a logic error
						if (!PerDeviceSdkInfo.Contains(DDPIDeviceId))
						{
							UE_LOGF(LogTurnkeySupport, Error, "Received DeviceId %ls from Turnkey, but the engine doesn't know about it.", *DDPIDeviceId);
							continue;
						}

						UE_LOGF(LogTurnkeySupport, Log, "Turnkey Device: %ls", *Line);

						PerDeviceSdkInfo[DDPIDeviceId] = SdkInfo;
					}
				}

			    for (const FString& Id : PlatformDeviceIds)
			    {
				    FTurnkeySdkInfo& SdkInfo = PerDeviceSdkInfo[ConvertToDDPIDeviceId(Id)];
				    if (SdkInfo.Status == ETurnkeyPlatformSdkStatus::Querying)
				    {
					    SdkInfo.Status = ETurnkeyPlatformSdkStatus::Error;
					    SdkInfo.SdkErrorInformation = NSLOCTEXT("Turnkey", "TurnkeyError_DeviceNotReturned", "A device's Sdk status was not returned from Turnkey");
				    }
			    }
			}
			else if (!IsEngineExitRequested())
			{
			    for (const FString& Id : PlatformDeviceIds)
			    {
				    FTurnkeySdkInfo& SdkInfo = PerDeviceSdkInfo[ConvertToDDPIDeviceId(Id)];
				    SdkInfo.Status = ETurnkeyPlatformSdkStatus::Error;
					SdkInfo.SdkErrorInformation = FText::Format(NSLOCTEXT("Turnkey", "TurnkeyError_ReturnedError", "Turnkey returned an error, code {0} (See log)"), { ExitCode });
				}
				UE_LOGF(LogTurnkeySupport, Warning, "Turnkey failed to run properly, full Turnkey output:\n%ls", *TurnkeyProcess->GetFullOutputWithoutDelegate());
			}


			// cleanup
			delete TurnkeyProcess;
			IFileManager::Get().Delete(*ReportFilename);
		});
	});

	TurnkeyProcess->Launch();
}

/**
 * Runs Turnkey to get the Sdk information for all known platforms
 */
void FTurnkeySupportModule::RepeatQuickLaunch(FString DeviceId)
{
	UE_LOGF(LogTurnkeySupport, Display, "Launching on %ls", *DeviceId);

	ITargetDeviceServicesModule* TargetDeviceServicesModule = static_cast<ITargetDeviceServicesModule*>(FModuleManager::Get().LoadModule(TEXT("TargetDeviceServices")));
	TSharedPtr<ITargetDeviceProxy> Proxy = TargetDeviceServicesModule->GetDeviceProxyManager()->FindProxyDeviceForTargetDevice(DeviceId);

	if (Proxy.IsValid())
	{				
		if (Proxy->IsSimulated())
		{
			HandleLaunchOnSimulatorActionExecute(DeviceId, Proxy->GetName(), true, Proxy->GetArchitecture());
		}
		else
		{
			HandleLaunchOnDeviceActionExecute(DeviceId, Proxy->GetName(), true, Proxy->GetArchitecture());
		}
	}
	else
	{
		// @todo show error toast
	}
}


FTurnkeySdkInfo FTurnkeySupportModule::GetSdkInfo(FName PlatformName, bool bBlockIfQuerying) const
{
	FScopeLock Lock(&GTurnkeySection);

	// return the status, or Unknown info if not known
	return PerPlatformSdkInfo.FindRef(ConvertToDDPIPlatform(PlatformName));
}

FTurnkeySdkInfo FTurnkeySupportModule::GetSdkInfoForDeviceId(const FString& DeviceId) const
{
#if PLATFORM_MAC_ARM64
	// Dummy info for fake local mac device
	if (DeviceId == TEXT("IOS@") + LocalMacDummyUUID)
	{
		FTurnkeySdkInfo DummyInfo;
		DummyInfo.Status = ETurnkeyPlatformSdkStatus::Valid;
		DummyInfo.DeviceStatus = ETurnkeyDeviceStatus::SoftwareValid;
		DummyInfo.DeviceAutoSoftwareUpdates = ETurnkeyDeviceAutoSoftwareUpdateMode::Unknown;
		DummyInfo.bCanInstallFullSdk = false;
		DummyInfo.bCanInstallAutoSdk = false;
		DummyInfo.bHasBestSdk = true;
		return DummyInfo;
	}
#endif
	
	FScopeLock Lock(&GTurnkeySection);

	// return the status, or Unknown info if not known
	return PerDeviceSdkInfo.FindRef(ConvertToDDPIDeviceId(DeviceId));
}

void FTurnkeySupportModule::ClearDeviceStatus(FName PlatformName)
{
	FScopeLock Lock(&GTurnkeySection);

	FString Prefix = ConvertToDDPIPlatform(PlatformName.ToString()) + "@";
	for (auto& Pair : PerDeviceSdkInfo)
	{
		if (PlatformName == NAME_None || Pair.Key.StartsWith(Prefix))
		{
			Pair.Value.Status = ETurnkeyPlatformSdkStatus::Unknown;
		}
	}
}



#if WITH_EDITOR
bool FTurnkeySupportModule::Exec_Editor( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar )
{
	if (FParse::Command( &Cmd, TEXT("Turnkey")) )
	{
		// run Turnkey via UAT. The Cmd is added at the end in case the user wants to run additional commands
		const FString CommandLine = FString::Printf( TEXT("Turnkey %s %s %s"), *ITurnkeyIOModule::Get().GetUATParams(), *FTurnkeyEditorSupport::GetUATOptions(), Cmd );
		FTurnkeyEditorSupport::RunUAT(CommandLine, FText::GetEmpty(), LOCTEXT("Turnkey_CustomTurnkeyName", "Executing Turnkey"), LOCTEXT("Turnkey_CustomTurnkeyShortName", "Turnkey"), FAppStyle::Get().GetBrush(TEXT("MainFrame.PackageProject")));
		return true;
	}
	else if ( FParse::Command( &Cmd, TEXT("RunUAT")) )
	{
		// run UAT directly. The Cmd is added at the start on the assumption that it contains the command to run
		const FString CommandLine = FString::Printf( TEXT("%s %s %s"), Cmd, *ITurnkeyIOModule::Get().GetUATParams(), *FTurnkeyEditorSupport::GetUATOptions() );
		FTurnkeyEditorSupport::RunUAT(CommandLine, FText::GetEmpty(), LOCTEXT("Turnkey_CustomUATName", "Executing Custom UAT Task"), LOCTEXT("Turnkey_CustomUATShortName", "UAT"), FAppStyle::Get().GetBrush(TEXT("MainFrame.PackageProject")));
		return true;
	}

	return false;
}
#endif


void FTurnkeySupportModule::StartupModule( )
{
	

}


void FTurnkeySupportModule::ShutdownModule( )
{
}


IMPLEMENT_MODULE(FTurnkeySupportModule, TurnkeySupport);

#undef LOCTEXT_NAMESPACE

#endif // UE_WITH_TURNKEY_SUPPORT
