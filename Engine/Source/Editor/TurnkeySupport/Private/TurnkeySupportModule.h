// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/ITurnkeySupportModule.h"

#if UE_WITH_TURNKEY_SUPPORT

#if WITH_EDITOR
	#include "Misc/CoreMisc.h"
#endif

/**
 * Editor main frame module
 */
class FTurnkeySupportModule	: public ITurnkeySupportModule
#if WITH_EDITOR
	, public FSelfRegisteringExec
#endif
{
public:

	virtual void MakeTurnkeyMenu(FToolMenuSection& MenuSection, const FToolMenuInsert& InsertPosition) const override;
	virtual void MakeQuickLaunchItems(class UToolMenu* Menu, FOnQuickLaunchSelected ExternalOnClickDelegate) const override;
	virtual void RepeatQuickLaunch(FString DeviceId) override;



	virtual void UpdateSdkInfo() override;
	void UpdateSdkInfoForAllDevices();
	void UpdateSdkInfoForProxy(const TSharedRef<class ITargetDeviceProxy>& AddedProxy);
	virtual void UpdateSdkInfoForDevices(TArray<FString> DeviceIds) override;

	virtual FTurnkeySdkInfo GetSdkInfo(FName PlatformName, bool bBlockIfQuerying = true) const override;
	virtual FTurnkeySdkInfo GetSdkInfoForDeviceId(const FString& DeviceId) const override;
	virtual void ClearDeviceStatus(FName PlatformName=NAME_None) override;
public:

	// IModuleInterface interface

	virtual void StartupModule( ) override;
	virtual void ShutdownModule( ) override;

	virtual bool SupportsDynamicReloading( ) override
	{
		return true; // @todo: Eventually, this should probably not be allowed.
	}


private:
	// Information about the validity of using a platform, discovered via Turnkey
	TMap<FName, FTurnkeySdkInfo> PerPlatformSdkInfo;

	// Information about the validity of each connected device (by string, discovered by Turnkey)
	TMap<FString, FTurnkeySdkInfo> PerDeviceSdkInfo;


	// menu helpers
	TSharedRef<SWidget> MakeTurnkeyMenuWidget() const;

#if WITH_EDITOR
	// FSelfRegisteringExec interface
	virtual bool Exec_Editor( class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar ) override;
#endif

};

#endif // UE_WITH_TURNKEY_SUPPORT
