// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/ITurnkeySupportModule.h"

#if UE_WITH_TURNKEY_SUPPORT

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Trace/Detail/Channel.h"
#include "TurnkeySupport.h"
#include "UObject/NameTypes.h"

FString ConvertToDDPIPlatform(const FString& Platform)
{
	FString New = Platform.Replace(TEXT("Editor"), TEXT("")).Replace(TEXT("Client"), TEXT("")).Replace(TEXT("Server"), TEXT(""));
	if (New == TEXT("Win64"))
	{
		New = TEXT("Windows");
	}
	return New;
}

FName ConvertToDDPIPlatform(const FName& Platform)
{
	return FName(*ConvertToDDPIPlatform(Platform.ToString()));
}

FString ConvertToUATPlatform(const FString& Platform)
{
	FString New = ConvertToDDPIPlatform(Platform);
	if (New == TEXT("Windows"))
	{
		New = TEXT("Win64");
	}
	return New;
}

FString ConvertToUATDeviceId(const FString& DeviceId)
{
	TArray<FString> PlatformAndDevice;
	int32 NumElems = DeviceId.ParseIntoArray(PlatformAndDevice, TEXT("@"), true);
	if(NumElems < 2)
	{
		UE_LOGF(LogTurnkeySupport, Fatal, "Badly formatted deviceId: %ls", *DeviceId);
	}

	return FString::Printf(TEXT("%s@%s"), *ConvertToUATPlatform(PlatformAndDevice[0]), *PlatformAndDevice[1]);
}

FString ConvertToDDPIDeviceId(const FString& DeviceId)
{
	TArray<FString> PlatformAndDevice;
	int32 NumElems = DeviceId.ParseIntoArray(PlatformAndDevice, TEXT("@"), true);
	if(NumElems < 2)
	{
		UE_LOGF(LogTurnkeySupport, Fatal, "Badly formatted deviceId: %ls", *DeviceId);
	}

	return FString::Printf(TEXT("%s@%s"), *ConvertToDDPIPlatform(PlatformAndDevice[0]), *PlatformAndDevice[1]);
}

#endif // UE_WITH_TURNKEY_SUPPORT