// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Modules/ModuleManager.h"

#ifndef UE_WITH_TURNKEY_SUPPORT
#define UE_WITH_TURNKEY_SUPPORT 1
#endif

#if UE_WITH_TURNKEY_SUPPORT

struct FToolMenuInsert;
struct FToolMenuSection;
class SWidget;

enum class ETurnkeyPlatformSdkStatus : uint8
{
	Unknown,
	Querying,
	Valid,
	OutOfDate,
	NoSdk,
	Error,
	// @todo turnkey: add AutoSdkValid and ManualSdkValid, with Valid a Combination of both
};

enum class ETurnkeyDeviceStatus : uint8
{
	Unknown,
	InvalidPrerequisites,
	SoftwareValid,
	SoftwareInvalid,
};

enum class ETurnkeyDeviceAutoSoftwareUpdateMode : uint8
{
	Unknown,
	Disabled,
	Enabled,
};

struct FTurnkeySdkInfo
{
	struct Version { FString Min, Max, Current; };
	ETurnkeyPlatformSdkStatus Status = ETurnkeyPlatformSdkStatus::Unknown;
	ETurnkeyDeviceStatus DeviceStatus = ETurnkeyDeviceStatus::Unknown;
	ETurnkeyDeviceAutoSoftwareUpdateMode DeviceAutoSoftwareUpdates = ETurnkeyDeviceAutoSoftwareUpdateMode::Unknown;
	FText SdkErrorInformation;
	TMap<FString, Version> SDKVersions;
	bool bCanInstallFullSdk;
	bool bCanInstallAutoSdk;
	bool bHasBestSdk;
};



DECLARE_DELEGATE_OneParam(FOnQuickLaunchSelected, FString);


/**
 * Interface for turnkey support module
 */
class ITurnkeySupportModule
	: public IModuleInterface
{
public:

	/**
	 * Make a Platforms menu in the given MenuSection at the specified InsertPosition
	 */
	virtual void MakeTurnkeyMenu(FToolMenuSection& MenuSection, const FToolMenuInsert& InsertPosition) const = 0;

	/**
	 * Make menu items for Quick Launch, so they can be added to the Play menu
	 */
	virtual void MakeQuickLaunchItems(class UToolMenu* Menu, FOnQuickLaunchSelected ExternalOnClickDelegate) const = 0;

	/**
	 * Runs Turnkey to get the Sdk information for all known platforms
	 */
	virtual void RepeatQuickLaunch(FString DeviceId) = 0;

	/**
	 * Runs Turnkey to get the Sdk information for all known platforms
	 */
	virtual void UpdateSdkInfo() = 0;

	/**
	 * Runs Turnkey to get the Sdk information for a list of devices
	 */
	virtual void UpdateSdkInfoForDevices(TArray<FString> DeviceIds) = 0;


	virtual FTurnkeySdkInfo GetSdkInfo(FName PlatformName, bool bBlockIfQuerying = true) const = 0;
	virtual FTurnkeySdkInfo GetSdkInfoForDeviceId(const FString& DeviceId) const = 0;
	// @todo turnkey look into remove this
	virtual void ClearDeviceStatus(FName PlatformName=NAME_None) = 0;

public:

	/**
	 * Gets a reference to the search module instance.
	 *
	 * @todo gmp: better implementation using dependency injection.
	 * @return A reference to the MainFrame module.
	 */
	static ITurnkeySupportModule& Get( )
	{
		static const FName TurnkeySupportModuleName = "TurnkeySupport";
		return FModuleManager::LoadModuleChecked<ITurnkeySupportModule>(TurnkeySupportModuleName);
	}

public:

	/**
	 * Virtual destructor.
	 */
	virtual ~ITurnkeySupportModule( ) = default;
};

#endif // UE_WITH_TURNKEY_SUPPORT
