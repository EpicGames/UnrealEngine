// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Interfaces/ITurnkeySupportModule.h"

#if UE_WITH_TURNKEY_SUPPORT

/* Public Dependencies
 *****************************************************************************/

#include "Modules/ModuleInterface.h"
#include "SlateFwd.h"
#include "Widgets/SWidget.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxExtender.h"


/* Public Includes
 *****************************************************************************/




DECLARE_LOG_CATEGORY_EXTERN(LogTurnkeySupport, Log, All);

FString ConvertToDDPIPlatform(const FString& Platform);
FName ConvertToDDPIPlatform(const FName& Platform);
FString ConvertToUATPlatform(const FString& Platform);
FString ConvertToUATDeviceId(const FString& DeviceId);
FString ConvertToDDPIDeviceId(const FString& DeviceId);

#endif // UE_WITH_TURNKEY_SUPPORT