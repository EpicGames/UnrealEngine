// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class TurnkeySupport : ModuleRules
{
	public TurnkeySupport(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"ApplicationCore",
				"DeveloperToolSettings",
				"EngineSettings",
				"InputCore",
				"RHI",
				"RenderCore",
				"Slate",
				"SlateCore",
				"TargetPlatform",
				"DesktopPlatform",
				"MessageLog",
 				"Projects",
 				"ToolMenus",
 				"LauncherServices",
				"SourceControl",
				"TurnkeyIO",
				"Analytics",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"DerivedDataCache",
			}
		);

		if (Target.bCompileAgainstEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
				"UnrealEd",
				"UATHelper",
 				"SettingsEditor",
				"Zen",
				}
			);
			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"GameProjectGeneration",
					"ProjectTargetPlatformEditor",
					"LevelEditor",
					"Settings",
	 				"MainFrame",
				}
			);
		}
	}
}
